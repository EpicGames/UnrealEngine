[CoreRedirects]
+ClassRedirects=(OldName="/Script/PCGGeometryScriptInterop.PCGSplineCrossSectionGeneratorSettings",NewName="/Script/PCGGeometryScriptInterop.PCGPrimitiveCrossSectionSettings")
+ClassRedirects=(OldName="/Script/PCGGeometryScriptInterop.PCGStaticMeshToDynamicMeshSettings",NewName="/Script/PCGGeometryScriptInterop.PCGMeshToDynamicMeshSettings")

+PropertyRedirects=(OldName="PCGMeshSamplerSettings.StaticMesh",NewName="PCGMeshSamplerSettings.Mesh")
+PropertyRedirects=(OldName="PCGAppendMeshesFromPointsSettings.StaticMesh",NewName="PCGAppendMeshesFromPointsSettings.Mesh")
+PropertyRedirects=(OldName="PCGMeshToDynamicMeshSettings.StaticMesh",NewName="PCGMeshToDynamicMeshSettings.Mesh")