// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGGeometryScriptInterop : ModuleRules
	{
		public PCGGeometryScriptInterop(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"GeometryScriptingCore",
					"Projects",
					"RenderCore",
					"RHI",
					"PCG",
					"ModelingOperators"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DynamicMesh",
					"GeometryAlgorithms",
					"GeometryCore",
					"GeometryFramework",
					"ModelingComponents"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"AdvancedPreviewScene",
						"UnrealEd",
						"PCGEditor"
					}
				);
			}
		}
	}
}
