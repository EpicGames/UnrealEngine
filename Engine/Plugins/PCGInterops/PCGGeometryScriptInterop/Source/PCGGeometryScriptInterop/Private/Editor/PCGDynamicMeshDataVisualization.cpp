// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/PCGDynamicMeshDataVisualization.h"

#if WITH_EDITOR

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"
#include "Helpers/PCGHelpers.h"
#include "Resources/PCGDynamicMeshManagedComponent.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "UDynamicMesh.h"
#include "Components/DynamicMeshComponent.h"

#define LOCTEXT_NAMESPACE "PCGDynamicMeshDataVisualization"

void FPCGDynamicMeshDataVisualization::ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const
{
	const UPCGDynamicMeshData* DynMeshData = CastChecked<UPCGDynamicMeshData>(Data);
	UPCGComponent* SourcePCGComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	if (!DynMeshData || !SourcePCGComponent)
	{
		return;
	}

	// We force debug resources to be transient.
	UPCGDynamicMeshManagedComponent* ManagedComponent = PCGDynamicMeshManagedComponent::GetOrCreateDynamicMeshManagedComponent(Context, SettingsInterface, DynMeshData, TargetActor, EPCGEditorDirtyMode::Preview);
	UDynamicMeshComponent* Component = ManagedComponent ? ManagedComponent->GetComponent() : nullptr;
	
	if (Component)
	{
		// Modifying the dynamic mesh component would trigger a refresh, so make sure we notify the PCG Component to ignore any changes to the dyn mesh component during that time.
		SourcePCGComponent->IgnoreChangeOriginDuringGenerationWithScope(Component, [&DynMeshData, &Component]() { DynMeshData->InitializeDynamicMeshComponentFromData(Component); });
	}
}

FPCGSetupSceneFunc FPCGDynamicMeshDataVisualization::GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const
{
	return [this, WeakData=TWeakObjectPtr<const UPCGDynamicMeshData>(Cast<UPCGDynamicMeshData>(Data))](FPCGSceneSetupParams& InOutParams)
	{
		check(InOutParams.Scene);
		check(InOutParams.EditorViewportClient);

		if (!WeakData.IsValid())
		{
			UE_LOGF(LogPCG, Error, "Failed to setup data viewport, the data was lost or invalid.");
			return;
		}

		TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = NewObject<UDynamicMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		WeakData->InitializeDynamicMeshComponentFromData(DynamicMeshComponent);

		InOutParams.ManagedResources.Add(DynamicMeshComponent);

		if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
		{
			DynamicMeshComponent->SetMobility(EComponentMobility::Static);
		}

		InOutParams.Scene->AddComponent(DynamicMeshComponent, FTransform::Identity);
		InOutParams.FocusBounds = DynamicMeshComponent->CalcLocalBounds();
	};
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR