// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "PCGDataVisualization.h"
#include "DataVisualizations/PCGSpatialDataVisualization.h"

class AActor;
class UPCGData;
struct FPCGContext;

class FPCGDynamicMeshDataVisualization : public IPCGSpatialDataVisualization
{
public:
	// ~Begin IPCGDataVisualization interface
	virtual void ExecuteDebugDisplay(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data, AActor* TargetActor) const override;
	virtual FPCGSetupSceneFunc GetViewportSetupFunc(const UPCGSettingsInterface* SettingsInterface, const UPCGData* Data) const override;
	// ~End IPCGDataVisualization interface
};

#endif // WITH_EDITOR