// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGAppendMeshesFromPoints.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/PCGBasePointData.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Helpers/PCGGeometryHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "DynamicMeshEditor.h"
#include "UDynamicMesh.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "DynamicMesh/MeshIndexMappings.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryScript/MeshAssetFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGAppendMeshesFromPoints)

#define LOCTEXT_NAMESPACE "PCGAppendMeshesFromPointsElement"

namespace PCGAppendMeshesFromPoints
{
	static const FName InDynMeshPinLabel = TEXT("InDynMesh");
	static const FName InPointsPinLabel = TEXT("InPoints");
	static const FName InAppendMeshPinLabel = TEXT("AppendDynMesh");
}

#if WITH_EDITOR
FName UPCGAppendMeshesFromPointsSettings::GetDefaultNodeName() const
{
	return FName(TEXT("AppendMeshesFromPoints"));
}

FText UPCGAppendMeshesFromPointsSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Append Meshes From Points");
}

FText UPCGAppendMeshesFromPointsSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Append meshes at the points transforms. Mesh can be a single static mesh, multiple meshes coming from the points or another dynamic mesh.");
}

void UPCGAppendMeshesFromPointsSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (Mode != EPCGAppendMeshesFromPointsMode::SingleStaticMesh || Mesh.IsNull() || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGAppendMeshesFromPointsSettings, Mesh)))
	{
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Mesh.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(MoveTemp(Key)).Emplace(this, /*bCulling=*/false);
}

EPCGChangeType UPCGAppendMeshesFromPointsSettings::GetChangeTypeForProperty(const FName& InPropertyName) const
{
	EPCGChangeType ChangeType = Super::GetChangeTypeForProperty(InPropertyName) | EPCGChangeType::Cosmetic;

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UPCGAppendMeshesFromPointsSettings, Mode))
	{
		// If we change from/to DynamicMesh, this needs to trigger a graph recompilation.
		ChangeType |= EPCGChangeType::Structural;
	}

	return ChangeType;
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGAppendMeshesFromPointsSettings::CreateElement() const
{
	return MakeShared<FPCGAppendMeshesFromPointsElement>();
}

TArray<FPCGPinProperties> UPCGAppendMeshesFromPointsSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGAppendMeshesFromPoints::InDynMeshPinLabel, EPCGDataType::DynamicMesh, false, false).SetRequiredPin();
	Properties.Emplace_GetRef(PCGAppendMeshesFromPoints::InPointsPinLabel, EPCGDataType::Point, false, false).SetRequiredPin();

	if (Mode == EPCGAppendMeshesFromPointsMode::DynamicMesh)
	{
		Properties.Emplace_GetRef(PCGAppendMeshesFromPoints::InAppendMeshPinLabel, EPCGDataType::DynamicMesh, false, false).SetRequiredPin();
	}
	
	return Properties;
}

TArray<FPCGPinProperties> UPCGAppendMeshesFromPointsSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::DynamicMesh, false, false);
	return Properties;
}

bool FPCGAppendMeshesFromPointsElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	// Without context, we can't know, so force it in the main thread to be safe.
	return !Context || Context->CurrentPhase == EPCGExecutionPhase::PrepareData;
}

FPCGContext* FPCGAppendMeshesFromPointsElement::CreateContext()
{
	return new FPCGAppendMeshesFromPointsContext();
}

bool FPCGAppendMeshesFromPointsElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAppendMeshesFromPointsElement::Execute);

	FPCGAppendMeshesFromPointsContext* Context = static_cast<FPCGAppendMeshesFromPointsContext*>(InContext);
	check(Context);
	
	const UPCGAppendMeshesFromPointsSettings* Settings = InContext->GetInputSettings<UPCGAppendMeshesFromPointsSettings>();
	check(Settings);
	
	if (Context->WasLoadRequested())
	{
		return true;
	}

	if (Settings->Mode == EPCGAppendMeshesFromPointsMode::SingleStaticMesh)
	{
		if (Settings->Mesh.IsNull())
		{
			return true;
		}
		
		Context->bPrepareDataSucceeded = true;
		return Context->RequestResourceLoad(Context, {Settings->Mesh.ToSoftObjectPath()}, !Settings->bSynchronousLoad);
	}

	if (Settings->Mode == EPCGAppendMeshesFromPointsMode::StaticMeshFromAttribute)
	{
		const TArray<FPCGTaggedData> InputPoints = InContext->InputData.GetInputsByPin(PCGAppendMeshesFromPoints::InPointsPinLabel);
		const UPCGBasePointData* InPointData = !InputPoints.IsEmpty() ? Cast<const UPCGBasePointData>(InputPoints[0].Data) : nullptr;

		if (!InPointData)
		{
			return true;
		}
		
		const FPCGAttributePropertyInputSelector Selector = Settings->MeshAttribute.CopyAndFixLast(InPointData);
		const TUniquePtr<const IPCGAttributeAccessor> Accessor = PCGAttributeAccessorHelpers::CreateConstAccessor(InPointData, Selector);
		const TUniquePtr<const IPCGAttributeAccessorKeys> Keys = PCGAttributeAccessorHelpers::CreateConstKeys(InPointData, Selector);

		if (!Accessor || !Keys)
		{
			PCGLog::Metadata::LogFailToCreateAccessorError(Selector, Context);
			return true;
		}

		if (!PCG::Private::IsBroadcastableOrConstructible(Accessor->GetUnderlyingType(), PCG::Private::MetadataTypes<FSoftObjectPath>::Id))
		{
			PCGLog::Metadata::LogFailToGetAttributeError<FSoftObjectPath>(Selector, Accessor.Get(), Context);
			return true;
		}

		TArray<FSoftObjectPath> StaticMeshesToLoad;
		PCGMetadataElementCommon::ApplyOnAccessor<FSoftObjectPath>(*Keys, *Accessor, [Context, &StaticMeshesToLoad](const FSoftObjectPath& Path, int32 Index)
		{
			if (!Path.IsNull())
			{
				StaticMeshesToLoad.AddUnique(Path);
				Context->MeshToPointIndicesMapping.FindOrAdd(Path).Add(Index);
			}
		}, EPCGAttributeAccessorFlags::AllowBroadcastAndConstructible);

		if (StaticMeshesToLoad.IsEmpty())
		{
			return true;
		}

		Context->bPrepareDataSucceeded = true;
		return Context->RequestResourceLoad(Context, std::move(StaticMeshesToLoad), !Settings->bSynchronousLoad);
	}
	
	Context->bPrepareDataSucceeded = true;
	return true;
}

bool FPCGAppendMeshesFromPointsElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGAppendMeshesFromPointsElement::Execute);

	FPCGAppendMeshesFromPointsContext* Context = static_cast<FPCGAppendMeshesFromPointsContext*>(InContext);
	check(Context);

	const UPCGAppendMeshesFromPointsSettings* Settings = InContext->GetInputSettings<UPCGAppendMeshesFromPointsSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> InputPoints = InContext->InputData.GetInputsByPin(PCGAppendMeshesFromPoints::InPointsPinLabel);
	const TArray<FPCGTaggedData> InputDynMesh = InContext->InputData.GetInputsByPin(PCGAppendMeshesFromPoints::InDynMeshPinLabel);
	const TArray<FPCGTaggedData> InputAppendDynMesh = InContext->InputData.GetInputsByPin(PCGAppendMeshesFromPoints::InAppendMeshPinLabel);

	const UPCGBasePointData* InPointData = !InputPoints.IsEmpty() ? Cast<const UPCGBasePointData>(InputPoints[0].Data) : nullptr;
	const UPCGDynamicMeshData* InDynMeshData = !InputDynMesh.IsEmpty() ? Cast<const UPCGDynamicMeshData>(InputDynMesh[0].Data) : nullptr;
	const UPCGDynamicMeshData* InAppendDynMeshData = !InputAppendDynMesh.IsEmpty() ? Cast<const UPCGDynamicMeshData>(InputAppendDynMesh[0].Data) : nullptr;

	if (!InPointData || !InDynMeshData || !Context->bPrepareDataSucceeded)
	{
		InContext->OutputData.TaggedData = InputDynMesh;
		return true;
	}

	FPCGTaggedData& OutputData = InContext->OutputData.TaggedData.Emplace_GetRef(InputDynMesh[0]);
	if (InPointData->IsEmpty())
	{
		return true;
	}

	UPCGDynamicMeshData* OutDynMeshData = CopyOrSteal(InputDynMesh[0], InContext);
	if (!OutDynMeshData)
	{
		return true;
	}
	
	OutputData.Data = OutDynMeshData;

	TMap<FSoftObjectPath, TUniquePtr<UE::Geometry::FDynamicMesh3>> MeshToDynMesh;

	// Temporary Dynamic Mesh object to be able to use extractor functions for Skeletal Meshes that only have a UDynamicMesh version
	UDynamicMesh* TempDynMesh = FPCGContext::NewObject_AnyThread<UDynamicMesh>(InContext);
	UGeometryScriptDebug* Debug = FPCGContext::NewObject_AnyThread<UGeometryScriptDebug>(InContext);

	auto ConvertStaticMesh = [Settings, &MeshToDynMesh, InContext, &OutDynMeshData, &TempDynMesh, Debug](const TSoftObjectPtr<UObject>& InMesh) -> bool
	{
		const FGeometryScriptMeshReadLOD MeshReadLOD{ Settings->RequestedLODType, Settings->RequestedLODIndex };
		FGeometryScriptCopyMeshFromAssetOptions Options{};

		UObject* Mesh = InMesh.LoadSynchronous();
		UStaticMesh* StaticMesh = Cast<UStaticMesh>(Mesh);
		USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Mesh);

		EGeometryScriptOutcomePins Outcome = EGeometryScriptOutcomePins::Failure;

		if (StaticMesh)
		{
			UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromStaticMeshV2(StaticMesh, TempDynMesh, Options, MeshReadLOD, Outcome, /*bUseSectionMaterials=*/false, Debug);
		}
		else if (SkeletalMesh)
		{
			UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromSkeletalMesh(SkeletalMesh, TempDynMesh, Options, MeshReadLOD, Outcome, Debug);
		}

		if (Outcome != EGeometryScriptOutcomePins::Success)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ObjectToDynamicMeshFailed", "Object to Dynamic mesh failed for object {0}. Either it is not a supported type (StaticMesh/SkeletalMesh) or there was a conversion issue."), FText::FromString(InMesh.ToString())));
			PCGGeometryHelpers::GeometryScriptDebugToPCGLog(InContext, Debug);
			return false;
		}
			
		TUniquePtr<UE::Geometry::FDynamicMesh3>& NewMesh = MeshToDynMesh.Emplace(InMesh.ToSoftObjectPath(), TempDynMesh->ExtractMesh());

		// Then do the remapping if needed
		if (Settings->bExtractMaterials)
		{
			TArray<UMaterialInterface*> MeshMaterials;
			TArray<FName> MaterialSlotNames;

			check(StaticMesh || SkeletalMesh);

			if (StaticMesh)
			{
				UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromStaticMesh(StaticMesh, MeshMaterials, MaterialSlotNames);
			}
			else
			{
				UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromSkeletalMesh(SkeletalMesh, MeshMaterials, MaterialSlotNames);
			}

			if (!MeshMaterials.IsEmpty() && MeshMaterials != OutDynMeshData->GetMaterials())
			{
				PCGGeometryHelpers::RemapMaterials(*NewMesh, MeshMaterials, OutDynMeshData->GetMutableMaterials());
			}
		}

		return true;
	};

	switch (Settings->Mode)
	{
	case EPCGAppendMeshesFromPointsMode::SingleStaticMesh:
	{
		if (!ConvertStaticMesh(Settings->Mesh))
		{
			return true;
		}

#if WITH_EDITOR
		if (Context->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGAppendMeshesFromPointsSettings, Mesh)))
		{
			FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(Context, FPCGSelectionKey::CreateFromPath(Settings->Mesh.ToSoftObjectPath()), /*bCulled=*/false);
		}
#endif // WITH_EDITOR

		break;
	}
	case EPCGAppendMeshesFromPointsMode::StaticMeshFromAttribute:
	{
#if WITH_EDITOR
		FPCGDynamicTrackingHelper DynamicTracking;
		DynamicTracking.EnableAndInitialize(Context, Context->MeshToPointIndicesMapping.Num());
#endif // WITH_EDITOR

		MeshToDynMesh.Reserve(Context->MeshToPointIndicesMapping.Num());
	
		for (TPair<FSoftObjectPath, TArray<int32>>& It : Context->MeshToPointIndicesMapping)
		{
			if (!ConvertStaticMesh(TSoftObjectPtr<UObject>(It.Key)))
			{
				return true;
			}

#if WITH_EDITOR
		DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(It.Key), /*bCulled=*/false);
#endif // WITH_EDITOR
		}

#if WITH_EDITOR
		DynamicTracking.Finalize(Context);
#endif // WITH_EDITOR

		break;
	}
	case EPCGAppendMeshesFromPointsMode::DynamicMesh:
	{
		if (!InAppendDynMeshData)
		{
			PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::DynamicMesh, PCGAppendMeshesFromPoints::InAppendMeshPinLabel, Context);
			return true;
		}

		// Remap materials if needed
		const TArray<TObjectPtr<UMaterialInterface>>& InputMaterials = InAppendDynMeshData->GetMaterials();
		TArray<TObjectPtr<UMaterialInterface>>& OutputMaterials = OutDynMeshData->GetMutableMaterials();
		if (!InputMaterials.IsEmpty() && OutputMaterials != InputMaterials)
		{
			// If we have a remap to do, we will need to copy or steal the append mesh.
			UPCGDynamicMeshData* InAppendDynamicMeshDataMutable = CopyOrSteal(InputAppendDynMesh[0], InContext);
			PCGGeometryHelpers::RemapMaterials(InAppendDynamicMeshDataMutable->GetMutableDynamicMesh()->GetMeshRef(), InputMaterials, OutputMaterials);
			InAppendDynMeshData = InAppendDynamicMeshDataMutable;
		}

		break;
	}
	default:
		return true;
	}
	
	UE::Geometry::FMeshIndexMappings MeshIndexMappings;
	UE::Geometry::FDynamicMeshEditor Editor(OutDynMeshData->GetMutableDynamicMesh()->GetMeshPtr());

	auto AppendMesh = [&Editor, &MeshIndexMappings](const UE::Geometry::FDynamicMesh3* MeshToAppend, const FTransform& PointTransform)
	{
		Editor.AppendMesh(MeshToAppend, MeshIndexMappings,
		[&PointTransform](int, const FVector& Position) { return PointTransform.TransformPosition(Position); },
		[&PointTransform](int, const FVector& Normal)
		{
			const FVector& PointScale = PointTransform.GetScale3D();
			const double DetSign = FMathd::SignNonZero(PointScale.X * PointScale.Y * PointScale.Z); // we only need to multiply by the sign of the determinant, rather than divide by it, since we normalize later anyway
			const FVector SafeInversePointScale(PointScale.Y * PointScale.Z * DetSign, PointScale.X * PointScale.Z * DetSign, PointScale.X * PointScale.Y * DetSign);
			return PointTransform.TransformVectorNoScale((SafeInversePointScale * Normal).GetSafeNormal());
		});
	};

	const TConstPCGValueRange<FTransform> TransformRange = InPointData->GetConstTransformValueRange();

	if (Settings->Mode == EPCGAppendMeshesFromPointsMode::StaticMeshFromAttribute)
	{
		for (TPair<FSoftObjectPath, TArray<int32>>& It : Context->MeshToPointIndicesMapping)
		{
			const UE::Geometry::FDynamicMesh3* MeshToAppend = MeshToDynMesh[It.Key].Get();
			check(MeshToAppend);

			for (const int32 i : It.Value)
			{
				AppendMesh(MeshToAppend, TransformRange[i]);
			}
		}
	}
	else
	{
		const UE::Geometry::FDynamicMesh3* MeshToAppend = nullptr;
		if (Settings->Mode == EPCGAppendMeshesFromPointsMode::SingleStaticMesh)
		{
			MeshToAppend = MeshToDynMesh[Settings->Mesh.ToSoftObjectPath()].Get();
		}
		else
		{
			MeshToAppend = InAppendDynMeshData->GetDynamicMesh() ? InAppendDynMeshData->GetDynamicMesh()->GetMeshPtr() : nullptr;
		}

		if (!MeshToAppend)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidDynMeshToAppend", "Invalid Dynamic mesh to append"), Context);
			return true;
		}

		for (const FTransform& PointTransform : TransformRange)
		{
			AppendMesh(MeshToAppend, PointTransform);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
