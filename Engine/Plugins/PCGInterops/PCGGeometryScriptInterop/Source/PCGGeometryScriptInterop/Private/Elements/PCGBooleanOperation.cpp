// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGBooleanOperation.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"
#include "Utils/PCGLogErrors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGBooleanOperation)

#define LOCTEXT_NAMESPACE "PCGBooleanOperationElement"

namespace PCGBooleanOperation
{
	static const FName InputAPinLabel = TEXT("InA");
	static const FName InputBPinLabel = TEXT("InB");
}

#if WITH_EDITOR
FName UPCGBooleanOperationSettings::GetDefaultNodeName() const
{
	return FName(TEXT("BooleanOperation"));
}

FText UPCGBooleanOperationSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Boolean Operation");
}

FText UPCGBooleanOperationSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Boolean operation between dynamic meshes.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGBooleanOperationSettings::CreateElement() const
{
	return MakeShared<FPCGBooleanOperationElement>();
}

TArray<FPCGPinProperties> UPCGBooleanOperationSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGBooleanOperation::InputAPinLabel, EPCGDataType::DynamicMesh).SetRequiredPin();
	Properties.Emplace_GetRef(PCGBooleanOperation::InputBPinLabel, EPCGDataType::DynamicMesh).SetRequiredPin();
	return Properties;
}

bool FPCGBooleanOperationElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGBooleanOperationElement::Execute);

	check(InContext);

	const UPCGBooleanOperationSettings* Settings = InContext->GetInputSettings<UPCGBooleanOperationSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> InputsA = InContext->InputData.GetInputsByPin(PCGBooleanOperation::InputAPinLabel);
	const TArray<FPCGTaggedData> InputsB = InContext->InputData.GetInputsByPin(PCGBooleanOperation::InputBPinLabel);

	if (InputsA.IsEmpty() || InputsB.IsEmpty())
	{
		return true;
	}

	if (Settings->Mode == EPCGBooleanOperationMode::EachAWithEachB && InputsA.Num() != 1 && InputsB.Num() != 1 && InputsA.Num() != InputsB.Num())
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("MismatchNumInputs", "There is a mismatch between the number of inputs. If BoolEachAWithEveryB is false, we only support N:1, 1:N and N:N operations"), InContext);
		return true;
	}
	
	// We can only steal the input if input A is not used multiple times.
	const bool bCanStealInput = InputsB.Num() == 1 || (Settings->Mode == EPCGBooleanOperationMode::EachAWithEachBSequentially)
		|| (Settings->Mode == EPCGBooleanOperationMode::EachAWithEachB && InputsA.Num() == InputsB.Num());
	
	const int32 NumIterations = Settings->Mode == EPCGBooleanOperationMode::EachAWithEachB ? FMath::Max(InputsA.Num(), InputsB.Num()) : InputsA.Num() * InputsB.Num();
	UPCGDynamicMeshData* CurrentOutputMeshData = nullptr;
	FPCGTaggedData* CurrentTaggedOutputData = nullptr;

	for (int32 i = 0; i < NumIterations; ++i)
	{
		const int32 InputAIndex = Settings->Mode == EPCGBooleanOperationMode::EachAWithEachB ? (i % InputsA.Num()) : (i / InputsB.Num());
		const int32 InputBIndex = i % InputsB.Num();
		
		const FPCGTaggedData& InputA = InputsA[InputAIndex];
		const FPCGTaggedData& InputB = InputsB[InputBIndex];

		const UPCGDynamicMeshData* InputMeshA = Cast<const UPCGDynamicMeshData>(InputA.Data);
		const UPCGDynamicMeshData* InputMeshB = Cast<const UPCGDynamicMeshData>(InputB.Data);

		if (!InputMeshA || !InputMeshB)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			continue;
		}

		// At every loop, we update our current data form the input.
		// In the EachAWithEachBSequentially case, only do it at the beginning of a new cycle (when we start at the first B again).
		if (Settings->Mode != EPCGBooleanOperationMode::EachAWithEachBSequentially || InputBIndex == 0)
		{
			CurrentOutputMeshData = bCanStealInput ? CopyOrSteal(InputA, InContext) : CastChecked<UPCGDynamicMeshData>(InputMeshA->DuplicateData(InContext));
			CurrentTaggedOutputData = &InContext->OutputData.TaggedData.Emplace_GetRef(InputA);
			CurrentTaggedOutputData->Data = CurrentOutputMeshData;
		}

		check(CurrentOutputMeshData);
		check(CurrentTaggedOutputData);

		// Second mesh is required to be non const, but it won't be modified (Geometry Script API is not const friendly), hence the const_cast.
		UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshBoolean(CurrentOutputMeshData->GetMutableDynamicMesh(), FTransform::Identity,
	const_cast<UDynamicMesh*>(InputMeshB->GetDynamicMesh()), FTransform::Identity,
			Settings->BooleanOperation, Settings->BooleanOperationOptions);

		
		if (Settings->TagInheritanceMode == EPCGBooleanOperationTagInheritanceMode::B)
		{
			CurrentTaggedOutputData->Tags = InputB.Tags;
		}
		else if (Settings->TagInheritanceMode == EPCGBooleanOperationTagInheritanceMode::Both)
		{
			CurrentTaggedOutputData->Tags.Append(InputB.Tags);
		}
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
