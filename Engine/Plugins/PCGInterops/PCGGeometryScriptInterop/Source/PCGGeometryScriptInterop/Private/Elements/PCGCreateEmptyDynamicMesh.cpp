// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGCreateEmptyDynamicMesh.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGCreateEmptyDynamicMesh)

#define LOCTEXT_NAMESPACE "PCGCreateEmptyDynamicMeshElement"

#if WITH_EDITOR
FName UPCGCreateEmptyDynamicMeshSettings::GetDefaultNodeName() const
{
	return FName(TEXT("CreateEmptyDynamicMesh"));
}

FText UPCGCreateEmptyDynamicMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Create Empty Dynamic Mesh");
}

FText UPCGCreateEmptyDynamicMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Create an empty dynamic mesh data.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGCreateEmptyDynamicMeshSettings::CreateElement() const
{
	return MakeShared<FPCGCreateEmptyDynamicMeshElement>();
}

TArray<FPCGPinProperties> UPCGCreateEmptyDynamicMeshSettings::InputPinProperties() const
{
	return {};
}

TArray<FPCGPinProperties> UPCGCreateEmptyDynamicMeshSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::DynamicMesh, false, false);
	return Properties;
}

bool FPCGCreateEmptyDynamicMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGCreateEmptyDynamicMeshElement::Execute);

	check(InContext);

	const UPCGCreateEmptyDynamicMeshSettings* Settings = InContext->GetInputSettings<UPCGCreateEmptyDynamicMeshSettings>();
	check(Settings);
	
	InContext->OutputData.TaggedData.Emplace_GetRef().Data = FPCGContext::NewObject_AnyThread<UPCGDynamicMeshData>(InContext);

	return true;
}

#undef LOCTEXT_NAMESPACE
