// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDynamicMeshBaseElement.h"

#include "PCGContext.h"
#include "PCGData.h"
#include "PCGGraphExecutionInspection.h"
#include "Data/PCGDynamicMeshData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDynamicMeshBaseElement)

#define LOCTEXT_NAMESPACE "PCGDynamicMeshBaseElement"

namespace PCGDynamicMeshBaseElement
{
	static TAutoConsoleVariable<bool> CVarPCGDynamicMeshAllowSteal(
		TEXT("pcg.DynamamicMesh.AllowDataSteal"),
		true,
		TEXT("Allows to steal dynamic meshes pcg data, avoiding a copy when possible."));

	static TAutoConsoleVariable<int32> CVarPCGDynamicMeshStealVerbose(
		TEXT("pcg.DynamamicMesh.DataStealVerbose"),
		0,
		TEXT("Verbosity to track steal. 0 = None, 1 = Log, 2 = Log + Graph message"));
}

TArray<FPCGPinProperties> UPCGDynamicMeshBaseSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::DynamicMesh).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGDynamicMeshBaseSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::DynamicMesh);
	return Properties;
}

UPCGDynamicMeshData* IPCGDynamicMeshBaseElement::CopyOrSteal(const FPCGTaggedData& InTaggedData, FPCGContext* InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IPCGDynamicMeshBaseElement::CopyOrSteal);
	
	check(InContext);
	
	const UPCGDynamicMeshData* InData = Cast<const UPCGDynamicMeshData>(InTaggedData.Data);
	if (!InData)
	{
		return nullptr;
	}

	bool bCanSteal = !InTaggedData.bIsUsedMultipleTimes && PCGDynamicMeshBaseElement::CVarPCGDynamicMeshAllowSteal.GetValueOnAnyThread();

#if WITH_EDITOR
	// We can't steal when we are inspecting, otherwise it breaks the inspection cache.
	if (InContext->ExecutionSource.IsValid() && InContext->ExecutionSource->GetExecutionState().GetInspection().IsInspecting())
	{
		bCanSteal = false; 
	}
#endif // WITH_EDITOR
	
	if (!bCanSteal)
	{
		return CastChecked<UPCGDynamicMeshData>(InData->DuplicateData(InContext));
	}

	const int32 StealVerbose = PCGDynamicMeshBaseElement::CVarPCGDynamicMeshStealVerbose.GetValueOnAnyThread();
	if (StealVerbose != 0)
	{
		
#if !UE_BUILD_SHIPPING
		const FName OriginatingNodeName = InTaggedData.OriginatingNode.IsValid() ? InTaggedData.OriginatingNode->GetFName() : TEXT("*UnknownSource*");
#else
		const FName OriginatingNodeName = TEXT("*UnknownSource*");
#endif // !UE_BUILD_SHIPPING
		
		const FText Message = FText::Format(LOCTEXT("StealVerbose", "[STEAL DATA] Data, originating from node {0}, on pin {1} was stolen."),
			FText::FromName(OriginatingNodeName), FText::FromName(InTaggedData.Pin));
		if (StealVerbose == 1)
		{
			PCGE_LOG_C(Warning, LogOnly, InContext, Message);
		}
		else
		{
			PCGLog::LogWarningOnGraph(Message, InContext);
		}
	}

	return const_cast<UPCGDynamicMeshData*>(InData);
}

#undef LOCTEXT_NAMESPACE
