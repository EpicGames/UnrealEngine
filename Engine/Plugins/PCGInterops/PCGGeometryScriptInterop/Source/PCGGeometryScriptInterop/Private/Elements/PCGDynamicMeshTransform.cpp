// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGDynamicMeshTransform.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"

#include "UDynamicMesh.h"
#include "DynamicMesh/MeshTransforms.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDynamicMeshTransform)

#define LOCTEXT_NAMESPACE "PCGDynamicMeshTransformElement"

#if WITH_EDITOR
FName UPCGDynamicMeshTransformSettings::GetDefaultNodeName() const
{
	return FName(TEXT("DynamicMeshTransform"));
}

FText UPCGDynamicMeshTransformSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Dynamic Mesh Transform");
}

FText UPCGDynamicMeshTransformSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Apply a transform to all dynamic meshes.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGDynamicMeshTransformSettings::CreateElement() const
{
	return MakeShared<FPCGDynamicMeshTransformElement>();
}

bool FPCGDynamicMeshTransformElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGDynamicMeshTransformElement::Execute);

	check(InContext);

	const UPCGDynamicMeshTransformSettings* Settings = InContext->GetInputSettings<UPCGDynamicMeshTransformSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		UPCGDynamicMeshData* OutputData = CopyOrSteal(Input, InContext);
		if (!OutputData)
		{
			continue;
		}
		
		MeshTransforms::ApplyTransform(OutputData->GetMutableDynamicMesh()->GetMeshRef(), Settings->Transform);
		InContext->OutputData.TaggedData.Emplace_GetRef(Input).Data = OutputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
