// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGeometryBlueprintElement.h"

#include "Data/PCGDynamicMeshData.h"
#include "Elements/PCGDynamicMeshBaseElement.h"

#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGeometryBlueprintElement)

#define LOCTEXT_NAMESPACE "PCGGeometryBlueprintElement"

UPCGGeometryBlueprintElement::UPCGGeometryBlueprintElement()
	: UPCGBlueprintBaseElement()
{
	// Setup the element for basic dynamic mesh processing
	bIsCacheable = false;
	bHasDefaultInPin = false;
	bHasDefaultOutPin = false;

	CustomInputPins.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::DynamicMesh).SetRequiredPin();
	CustomOutputPins.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::DynamicMesh);
}

void UPCGGeometryBlueprintElement::Execute_Implementation(const FPCGDataCollection& Input, FPCGDataCollection& Output)
{
	// Verify that we are in the right setup
	const bool bHasASingleInputPin = (bHasDefaultInPin && CustomInputPins.IsEmpty()) || (!bHasDefaultInPin && CustomInputPins.Num() == 1);
	const bool bHasASingleOutputPin = (bHasDefaultOutPin && CustomOutputPins.IsEmpty()) || (!bHasDefaultOutPin && CustomOutputPins.Num() == 1);

	const bool bIsProcessDynMeshOverridden =  GetClass()->IsFunctionImplementedInScript(GET_FUNCTION_NAME_CHECKED(UPCGGeometryBlueprintElement, ProcessDynamicMesh));
	if (!bIsProcessDynMeshOverridden)
	{
		return;
	}

	if (!bHasASingleInputPin || !bHasASingleOutputPin)
	{
		// Make sure to throw a warning if we are in this case and Process Dynamic Mesh is overridden to warn the user their function won't be called
		PCGLog::LogWarningOnGraph(LOCTEXT("DynMeshOverridenButNotCalled", "Process Dynamic Mesh was overridden, but we don't have the expected setup (single input and output pin). Process Dynamic Mesh won't be called."), CurrentContext);
		return;
	}

	const FName InputPinLabel = bHasDefaultInPin ? PCGPinConstants::DefaultInputLabel : CustomInputPins[0].Label;
	const FName OutputPinLabel = bHasDefaultOutPin ? PCGPinConstants::DefaultOutputLabel : CustomOutputPins[0].Label;
	
	for (const FPCGTaggedData& InputData : Input.GetInputsByPin(InputPinLabel))
	{
		if (!Cast<const UPCGDynamicMeshData>(InputData.Data))
		{
			continue;
		}

		UPCGDynamicMeshData* ProcessingMesh = CopyOrStealInputData(InputData);
		TArray<FString> OutTags;

		ProcessDynamicMesh(ProcessingMesh->GetMutableDynamicMesh(), OutTags);
		FPCGTaggedData& OutputData = Output.TaggedData.Emplace_GetRef(InputData);
		OutputData.Tags.Append(OutTags);
		OutputData.Data = ProcessingMesh;
		OutputData.Pin = OutputPinLabel;
	}
}
UPCGDynamicMeshData* UPCGGeometryBlueprintElement::CopyOrStealInputData(const FPCGTaggedData& InTaggedData) const
{
	if (bIsCacheable || IsCacheableOverride())
	{
		// Verification that the user didn't change default settings
		PCGLog::LogWarningOnGraph(LOCTEXT("SettingsDifferent", "In PCG Geometry Blueprint Element, the default settings were changed (not cacheable and not verifying outputs used multiple times)."
			"Use the normal BP element if you want this behavior. Will always copy and never steal."), CurrentContext);
		const UPCGDynamicMeshData* InData = Cast<const UPCGDynamicMeshData>(InTaggedData.Data);
		return InData ? CastChecked<UPCGDynamicMeshData>(InData->DuplicateData(CurrentContext)) : nullptr;
	}
	
	return IPCGDynamicMeshBaseElement::CopyOrSteal(InTaggedData, CurrentContext);
}

#undef LOCTEXT_NAMESPACE
