// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetDynamicMeshData.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/Registry/PCGGetDataFunctionRegistry.h"
#include "Helpers/PCGGeometryHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "DynamicMeshActor.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetDynamicMeshData)

#define LOCTEXT_NAMESPACE "PCGGetDynamicMeshDataElement"

#if WITH_EDITOR
FName UPCGGetDynamicMeshDataSettings::GetDefaultNodeName() const
{
	return FName(TEXT("GetDynamicMeshData"));
}

FText UPCGGetDynamicMeshDataSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Dynamic Mesh Data");
}
#endif // WITH_EDITOR

TArray<FPCGPinProperties> UPCGGetDynamicMeshDataSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::DynamicMesh);

	return PinProperties;
}

bool PCGGetDynamicMeshData::GetDynamicMeshDataFromActor(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, AActor* InActor, FPCGGetDataFunctionRegistryOutput& Output)
{
	// Request Dynamic Mesh Data filter explicitly, otherwise it would consume too many actors/components.
	if (InParams.DataTypeFilter != EPCGDataType::DynamicMesh)
	{
		return false;
	}
	
	check(InActor);

	// Early out if the actor gets rejected by the component selector
	if (InParams.ComponentSelector && !InParams.ComponentSelector->FilterActor(InActor))
	{
		return false;
	}
	
	if (ADynamicMeshActor* DynMeshActor = Cast<ADynamicMeshActor>(InActor))
	{
		UDynamicMeshComponent* Component = DynMeshActor->GetDynamicMeshComponent();
		if (!Component || !Component->GetDynamicMesh())
		{
			return false;
		}
		
		auto NameTagsToStringTags = [](const FName& InName) { return InName.ToString(); };
		TSet<FString> ActorTags;
		Algo::Transform(InActor->Tags, ActorTags, NameTagsToStringTags);
		
		UPCGDynamicMeshData* Data = FPCGContext::NewObject_AnyThread<UPCGDynamicMeshData>(InContext);
		Data->Initialize(Component->GetDynamicMesh(), /*bCanTakeOwnership=*/false, Component->GetMaterials());

		FPCGTaggedData& TaggedData = Output.Collection.TaggedData.Emplace_GetRef();
		TaggedData.Data = Data;
		TaggedData.Tags = ActorTags;
		return true;
	}
	else
	{
		return false;
	}
}

bool PCGGetDynamicMeshData::GetDynamicMeshDataFromComponent(FPCGContext* InContext, const FPCGGetDataFunctionRegistryParams& InParams, UActorComponent* InActorComponent, FPCGGetDataFunctionRegistryOutput& Output)
{
	check(InActorComponent);

	// Request Dynamic Mesh Data filter explicitly, otherwise it would consume too many actors/components.
	if (InParams.DataTypeFilter != EPCGDataType::DynamicMesh)
	{
		return false;
	}

	if (InParams.bIgnorePCGGeneratedComponents && InActorComponent->ComponentTags.Contains(PCGHelpers::DefaultPCGTag))
	{
		return false;
	}
	
	// Check if it is a scene component and try to extract it.
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InActorComponent))
	{
		UPCGDynamicMeshData* OutputData = FPCGContext::NewObject_AnyThread<UPCGDynamicMeshData>(InContext);
		UDynamicMesh* NewDynamicMesh = OutputData->GetMutableDynamicMesh();
		check(NewDynamicMesh);

		const UPCGGetDynamicMeshDataSettings* Settings = InContext ? InContext->GetInputSettings<UPCGGetDynamicMeshDataSettings>() : nullptr;
		
		// Adaptation of UGeometryScriptLibrary_SceneUtilityFunctions::CopyMeshFromComponent, since we don't have access to the Material list with this one.
		FTransform Transform;
		FText ErrorMessage;
		UE::Conversion::FToMeshOptions Options{};
		if (Settings)
		{
			Options.bWantNormals = Settings->Options.bWantNormals;
			Options.bWantTangents = Settings->Options.bWantTangents;
			Options.bWantInstanceColors = Settings->Options.bWantInstanceColors;
			Options.LODType = PCGGeometryHelpers::SafeConversionLODType(Settings->Options.RequestedLOD.LODType);
			Options.LODIndex = Settings->Options.RequestedLOD.LODIndex;
		}

		TArray<UMaterialInterface*> ComponentMaterialList;
		TArray<UMaterialInterface*> AssetMaterialList;

		const bool bSuccess = UE::Conversion::SceneComponentToDynamicMesh(SceneComponent, Options, /*bTransformToWorld=*/false, NewDynamicMesh->GetMeshRef(), Transform, ErrorMessage, &ComponentMaterialList, &AssetMaterialList);
		if (bSuccess)
		{
			OutputData->SetMaterials(!ComponentMaterialList.IsEmpty() ? ComponentMaterialList : AssetMaterialList);
			
			auto NameTagsToStringTags = [](const FName& InName) { return InName.ToString(); };

			FPCGTaggedData& TaggedData = Output.Collection.TaggedData.Emplace_GetRef();
			TaggedData.Data = OutputData;
			
			Algo::Transform(SceneComponent->ComponentTags, TaggedData.Tags, NameTagsToStringTags);

			if (InParams.bAddActorTags && SceneComponent->GetOwner())
			{
				TSet<FString> ActorTags;
				Algo::Transform(SceneComponent->GetOwner()->Tags, ActorTags, NameTagsToStringTags);
				TaggedData.Tags.Append(ActorTags);
			}
		}
		else
		{
			// If it fails, we still return true so we don't falloff on default getter behavior. 
			PCGLog::LogErrorOnGraph(ErrorMessage, InContext);
		}

		return true;
	}
	else
	{
		return false;
	}
}

#undef LOCTEXT_NAMESPACE
