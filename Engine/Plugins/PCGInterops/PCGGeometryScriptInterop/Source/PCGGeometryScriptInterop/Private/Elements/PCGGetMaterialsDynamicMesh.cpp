// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGGetMaterialsDynamicMesh.h"

#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGDynamicMeshData.h"
#include "Helpers/PCGPropertyHelpers.h"

#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGGetMaterialsDynamicMesh)

#define LOCTEXT_NAMESPACE "PCGGetMaterialsDynamicMesh"

#if WITH_EDITOR
FName UPCGGetMaterialsDynamicMeshSettings::GetDefaultNodeName() const
{
	return FName(TEXT("GetMaterialsDynamicMesh"));
}

FText UPCGGetMaterialsDynamicMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Get Materials On Dynamic Mesh");
}

FText UPCGGetMaterialsDynamicMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Retrieve the array of materials on a dynamic mesh data.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGGetMaterialsDynamicMeshSettings::CreateElement() const
{
	return MakeShared<FPCGGetMaterialsDynamicMeshElement>();
}

TArray<FPCGPinProperties> UPCGGetMaterialsDynamicMeshSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, FPCGDataTypeInfoParam::AsId());
	return PinProperties;
}

bool FPCGGetMaterialsDynamicMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGGetMaterialsDynamicMeshElement::Execute);

	check(InContext);

	const UPCGGetMaterialsDynamicMeshSettings* Settings = InContext->GetInputSettings<UPCGGetMaterialsDynamicMeshSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPCGDynamicMeshData* InputData = Cast<const UPCGDynamicMeshData>(Input.Data);
		if (!InputData)
		{
			continue;
		}

		UPCGParamData* OutputData = FPCGContext::NewObject_AnyThread<UPCGParamData>(InContext);
		UPCGMetadata* OutMetadata = OutputData->MutableMetadata();
		FPCGMetadataAttribute<FSoftObjectPath>* OutAttribute = OutMetadata->CreateAttribute<FSoftObjectPath>(FName("Materials"), FSoftObjectPath{}, false, false);
		if (!ensure(OutAttribute))
		{
			continue;
		}

		for (const UMaterialInterface* Material : InputData->GetMaterials())
		{
			OutAttribute->SetValue(OutMetadata->AddEntry(), FSoftObjectPath{Material});
		}
		
		InContext->OutputData.TaggedData.Emplace_GetRef(Input).Data = OutputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
