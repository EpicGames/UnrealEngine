// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMergeDynamicMeshes.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"

#include "DynamicMeshEditor.h"
#include "UDynamicMesh.h"
#include "GeometryScript/MeshBasicEditFunctions.h"
#include "Helpers/PCGGeometryHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMergeDynamicMeshes)

#define LOCTEXT_NAMESPACE "PCGMergeDynamicMeshesElement"

#if WITH_EDITOR
FName UPCGMergeDynamicMeshesSettings::GetDefaultNodeName() const
{
	return FName(TEXT("MergeDynamicMeshes"));
}

FText UPCGMergeDynamicMeshesSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Merge Dynamic Meshes");
}

FText UPCGMergeDynamicMeshesSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Appends all incoming dynamic meshes to the first dynamic mesh in order.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGMergeDynamicMeshesSettings::CreateElement() const
{
	return MakeShared<FPCGMergeDynamicMeshesElement>();
}

TArray<FPCGPinProperties> UPCGMergeDynamicMeshesSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::DynamicMesh, false, false);
	return Properties;
}

bool FPCGMergeDynamicMeshesElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMergeDynamicMeshesElement::Execute);

	check(InContext);

	const UPCGMergeDynamicMeshesSettings* Settings = InContext->GetInputSettings<UPCGMergeDynamicMeshesSettings>();
	check(Settings);

	UPCGDynamicMeshData* OutputData = nullptr;

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPCGDynamicMeshData* InputData = Cast<const UPCGDynamicMeshData>(Input.Data);
		if (!InputData)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			continue;
		}

		if (!OutputData)
		{
			OutputData = CopyOrSteal(Input, InContext);
			InContext->OutputData.TaggedData.Emplace_GetRef(Input).Data = OutputData;
		}
		else
		{
			UE::Geometry::FMeshIndexMappings MeshIndexMappings;
			
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMergeDynamicMeshesElement::Execute::AppendMesh);
				
				UE::Geometry::FDynamicMeshEditor Editor(OutputData->GetMutableDynamicMesh()->GetMeshPtr());
				Editor.AppendMesh(InputData->GetDynamicMesh()->GetMeshPtr(), MeshIndexMappings);
			}

			// It's also important to re-map the materials, but it's needed only if we do not have the same materials
			const TArray<TObjectPtr<UMaterialInterface>>& InputMaterials = InputData->GetMaterials();
			TArray<TObjectPtr<UMaterialInterface>>& OutputMaterials = OutputData->GetMutableMaterials();
			if (!InputMaterials.IsEmpty() && InputMaterials != OutputMaterials)
			{
				PCGGeometryHelpers::RemapMaterials(OutputData->GetMutableDynamicMesh()->GetMeshRef(), InputMaterials, OutputMaterials, &MeshIndexMappings);
			}
		}
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
