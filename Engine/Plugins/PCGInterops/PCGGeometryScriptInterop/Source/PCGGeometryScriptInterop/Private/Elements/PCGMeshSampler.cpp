// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMeshSampler.h"

#include "Helpers/PCGGeometryHelpers.h"

#include "PCGComponent.h"
#include "PCGModule.h"
#include "PCGPin.h"
#include "Data/PCGPointData.h"
#include "Helpers/PCGAsync.h"
#include "Helpers/PCGDynamicTrackingHelpers.h"
#include "Helpers/PCGHelpers.h"
#include "Metadata/PCGMetadata.h"
#include "Metadata/PCGMetadataAttributeTpl.h"

#include "UDynamicMesh.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "Data/PCGDynamicMeshData.h"
#include "Engine/AssetManager.h"
#include "Engine/Blueprint.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "GeometryScript/MeshMaterialFunctions.h"
#include "GeometryScript/MeshNormalsFunctions.h"
#include "GeometryScript/MeshQueryFunctions.h"
#include "GeometryScript/MeshRepairFunctions.h"
#include "GeometryScript/MeshVertexColorFunctions.h"
#include "GeometryScript/MeshVoxelFunctions.h"
#include "GeometryScript/SceneUtilityFunctions.h"
#include "Sampling/MeshSurfacePointSampling.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMeshSampler)

#define LOCTEXT_NAMESPACE "PCGMeshSampler"

namespace PCGMeshSampler
{
	bool SampleOnePointPerVertex(const UPCGMeshSamplerSettings* Settings, FPCGMeshSamplerContext* Context, FPCGMeshSamplerContext::SetPointDensityFunc SetPointDensityPtr)
	{
		check(Settings && Context && SetPointDensityPtr);

		TRACE_CPUPROFILER_EVENT_SCOPE(PCGMeshSampler::SampleOnePointPerVertex);

		auto IterationBody = [Settings, Context, SetPointDensityPtr](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			int32 NumWritten = 0;
			int32 DataIndex = 0;

			auto UpdateDataIndex = [&DataIndex, Context](int32 InReadIndex) -> bool
			{
				bool bUpdated = false;
				while (!Context->DynamicMeshes[DataIndex] || InReadIndex >= Context->StartingIndices[DataIndex + 1])
				{
					DataIndex++;
					bUpdated = true;
				}
				return bUpdated;
			};

			UpdateDataIndex(StartReadIndex);
			FPCGPointValueRanges OutRanges(Context->OutputData[DataIndex], /*bAllocate=*/false);

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				if (UpdateDataIndex(ReadIndex))
				{
					OutRanges = FPCGPointValueRanges(Context->OutputData[DataIndex], /*bAllocate=*/false);
				}

				const TArray<FVector>& Positions = *Context->Positions[DataIndex].List.Get();
				const TArray<FLinearColor>& Colors = *Context->Colors[DataIndex].List.Get();
				const TArray<FVector>& Normals = *Context->Normals[DataIndex].List.Get();

				const int32 CurrentIndex = ReadIndex - Context->StartingIndices[DataIndex];

				const FVector& Position = Positions[CurrentIndex];
				const FLinearColor& Color = Colors[CurrentIndex];
				const FVector& Normal = Normals[CurrentIndex];

				FPCGPoint OutPoint{};
				OutPoint.Transform = FTransform{ FRotationMatrix::MakeFromZ(Normal).Rotator(), Position, FVector::OneVector };
				OutPoint.Color = Color;
				OutPoint.Steepness = Settings->PointSteepness;

				SetPointDensityPtr(Color, OutPoint);

				OutPoint.Seed = PCGHelpers::ComputeSeedFromPosition(OutPoint.Transform.GetLocation());

				OutRanges.SetFromPoint(CurrentIndex, OutPoint);
				++NumWritten;
			}

			check(Count == NumWritten);
			return Count;
		};

		// Small verification to catch any problem that could arise in the iteration body
		if (Context->DynamicMeshes.IsEmpty() || !ensure(Context->DynamicMeshes.Num() + 1 == Context->StartingIndices.Num()))
		{
			return true;
		}

		return FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, Context->StartingIndices.Last(), []() {}, IterationBody, /*bEnableTimeSlicing=*/true);
	}

	bool SampleOnePointPerTriangle(const UPCGMeshSamplerSettings* Settings, FPCGMeshSamplerContext* Context, FPCGMeshSamplerContext::SetPointDensityFunc SetPointDensityPtr)
	{
		check(Settings && Context && SetPointDensityPtr);

		TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMeshSamplerElement::Execute::OnePointPerTriangle);

		auto IterationBody = [Context, Settings, SetPointDensityPtr](int32 StartReadIndex, int32 StartWriteIndex, int32 Count)
		{
			int32 NumWritten = 0;
			int32 DataIndex = 0;

			auto UpdateDataIndex = [&DataIndex, Context](int32 InReadIndex) -> bool
			{
				bool bUpdated = false;
				while (!Context->DynamicMeshes[DataIndex] || InReadIndex >= Context->StartingIndices[DataIndex + 1])
				{
					DataIndex++;
					bUpdated = true;
				}
				return bUpdated;
			};

			UpdateDataIndex(StartReadIndex);
			FPCGPointValueRanges OutRanges(Context->OutputData[DataIndex], /*bAllocate=*/false);

			for (int32 ReadIndex = StartReadIndex; ReadIndex < StartReadIndex + Count; ++ReadIndex)
			{
				if (UpdateDataIndex(ReadIndex))
				{
					OutRanges = FPCGPointValueRanges(Context->OutputData[DataIndex], /*bAllocate=*/false);
				}

				const TArray<int32>& TriangleIds = *Context->TriangleIds[DataIndex].List.Get();
				const int32 CurrentIndex = ReadIndex - Context->StartingIndices[DataIndex];

				const int32 TriangleId = TriangleIds[CurrentIndex];

				FVector Vertex1, Vertex2, Vertex3;
				bool bIsValidTriangle;

				UGeometryScriptLibrary_MeshQueryFunctions::GetTrianglePositions(Context->DynamicMeshes[DataIndex], TriangleId, bIsValidTriangle, Vertex1, Vertex2, Vertex3);
				const FVector Normal = UGeometryScriptLibrary_MeshQueryFunctions::GetTriangleFaceNormal(Context->DynamicMeshes[DataIndex], TriangleId, bIsValidTriangle);
				const FVector Position = (Vertex1 + Vertex2 + Vertex3) / 3.0;

				FPCGPoint OutPoint{};
				OutPoint.Transform = FTransform{ FRotationMatrix::MakeFromZ(Normal).Rotator(), Position, FVector::OneVector };
				OutPoint.Steepness = Settings->PointSteepness;

				FVector Dummy1, Dummy2, Dummy3;
				FVector BarycentricCoord;
				bool bIsValid = false;
				UGeometryScriptLibrary_MeshQueryFunctions::ComputeTriangleBarycentricCoords(Context->DynamicMeshes[DataIndex], TriangleId, bIsValid, Position, Dummy1, Dummy2, Dummy3, BarycentricCoord);

				Context->SetPointColorAndDensity(SetPointDensityPtr, TriangleId, BarycentricCoord, OutPoint, DataIndex, Settings->bHasDefaultColorOverride ? Settings->DefaultColorOverride : TOptional<FLinearColor>{});
				Context->SetAttributeValues(Settings->UVChannel, TriangleId, BarycentricCoord, OutPoint.MetadataEntry, DataIndex);

				OutPoint.Seed = PCGHelpers::ComputeSeedFromPosition(OutPoint.Transform.GetLocation());

				OutRanges.SetFromPoint(CurrentIndex, OutPoint);
				++NumWritten;
			}

			check(Count == NumWritten);
			return Count;
		};

		// Small verification to catch any problem that could arise in the iteration body
		if (Context->DynamicMeshes.IsEmpty() || !ensure(Context->DynamicMeshes.Num() + 1 == Context->StartingIndices.Num()))
		{
			return true;
		}

		return FPCGAsync::AsyncProcessingOneToOneRangeEx(&Context->AsyncState, Context->StartingIndices.Last(), []() {}, IterationBody, /*bEnableTimeSlicing=*/true);
	}

	bool PoissonSampling(const UPCGMeshSamplerSettings* Settings, FPCGMeshSamplerContext* Context, FPCGMeshSamplerContext::SetPointDensityFunc SetPointDensityPtr)
	{
		check(Settings && Context && SetPointDensityPtr);

		// For Poisson sampling, we are calling a "all-in-one" function, where we don't have control for timeslicing.
		// Since Poisson sampling can be expensive (depending on the radius used), we will do the sampling in a future, put this task to sleep,
		// and wait for the sampling to wake us up.
		// @todo_pcg: convert to use ScheduleGeneric + ExecutionDependencies
		if (Context->SamplingFutures.IsEmpty())
		{
			// Put the task asleep
			Context->bIsPaused = true;
			Context->SamplingProgess = MakeUnique<FProgressCancel>();
			Context->SamplingProgess->CancelF = [Context]() -> bool { return Context->StopSampling; };

			for (int32 DataIndex = 0; DataIndex < Context->DynamicMeshes.Num(); ++DataIndex)
			{
				if (!Context->DynamicMeshes[DataIndex])
				{
					continue;
				}

				auto SamplingFuture = [Settings, Context, Seed = Context->GetSeed(), SetPointDensityPtr, DataIndex]() -> bool
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMeshSamplerElement::Execute::PoissonSampling);

					UE::Geometry::FMeshSurfacePointSampling PointSampling;
					PointSampling.SampleRadius = Settings->SamplingOptions.SamplingRadius;
					PointSampling.MaxSamples = Settings->SamplingOptions.MaxNumSamples;
					PointSampling.RandomSeed = PCGHelpers::ComputeSeed(Seed, Settings->SamplingOptions.RandomSeed);
					PointSampling.SubSampleDensity = Settings->SamplingOptions.SubSampleDensity;
					PointSampling.SamplingMethodVersion = Settings->SamplingOptions.SamplingMethodVersion;

					if (Settings->NonUniformSamplingOptions.MaxSamplingRadius > PointSampling.SampleRadius)
					{
						PointSampling.MaxSampleRadius = Settings->NonUniformSamplingOptions.MaxSamplingRadius;
						PointSampling.SizeDistribution = UE::Geometry::FMeshSurfacePointSampling::ESizeDistribution(static_cast<int>(Settings->NonUniformSamplingOptions.SizeDistribution));
						PointSampling.SizeDistributionPower = FMath::Clamp(Settings->NonUniformSamplingOptions.SizeDistributionPower, 1.0, 10.0);
					}

					PointSampling.bComputeBarycentrics = true;

					PointSampling.ComputePoissonSampling(Context->DynamicMeshes[DataIndex]->GetMeshRef(), Context->SamplingProgess.Get());

					if (Context->StopSampling)
					{
						return true;
					}

					TArray<FPCGPoint> Points;
					Points.Reserve(PointSampling.Samples.Num());

					int Count = 0;
					for (int32 i = 0; i < PointSampling.Samples.Num(); ++i)
					{
						UE::Geometry::FFrame3d& Sample = PointSampling.Samples[i];
						// Avoid to check too many times
						constexpr int CancelledCheckNum = 25;
						if (++Count == CancelledCheckNum)
						{
							Count = 0;
							if (Context->StopSampling)
							{
								return true;
							}
						}

						const int32 TriangleId = PointSampling.TriangleIDs[i];
						const FVector BarycentricCoords = PointSampling.BarycentricCoords[i];

						FPCGPoint& OutPoint = Points.Emplace_GetRef();
						OutPoint.Transform = Sample.ToTransform();
						OutPoint.Steepness = Settings->PointSteepness;

						Context->SetPointColorAndDensity(SetPointDensityPtr, TriangleId, BarycentricCoords, OutPoint, DataIndex, Settings->bHasDefaultColorOverride ? Settings->DefaultColorOverride : TOptional<FLinearColor>{});
						Context->SetAttributeValues(Settings->UVChannel, TriangleId, BarycentricCoords, OutPoint.MetadataEntry, DataIndex);

						OutPoint.Seed = PCGHelpers::ComputeSeedFromPosition(OutPoint.Transform.GetLocation());
					}

					Context->OutputData[DataIndex]->SetNumPoints(Points.Num(), /*bInitializeValues=*/false);
					// @todo_pcg: could probably be optimzed to save a little memory
					Context->OutputData[DataIndex]->AllocateProperties(EPCGPointNativeProperties::All);
					FPCGPointValueRanges OutRanges(Context->OutputData[DataIndex], /*bAllocate=*/false);
					for (int32 PointIndex = 0; PointIndex < Points.Num(); ++PointIndex)
					{
						OutRanges.SetFromPoint(PointIndex, Points[PointIndex]);
					}

					// Unpause the task
					Context->bIsPaused = false;
					return true;
				};

				Context->SamplingFutures.Add(Async(EAsyncExecution::ThreadPool, std::move(SamplingFuture)));
			}
		}

		const bool bIsDone = Algo::AllOf(Context->SamplingFutures, [](const TFuture<bool>& Future) { return Future.IsReady();});
		if (bIsDone)
		{
			for (TFuture<bool>& Future : Context->SamplingFutures)
			{
				Future.Reset();
			}
		}

		return bIsDone;
	}
}

void FPCGMeshSamplerContext::SetAttributeValues(int32 UVChannel, int32 TriangleId, const FVector& BarycentricCoord, int64& MetadataEntry, int32 DataIndex)
{
	if (UVAttributes.IsValidIndex(DataIndex) && UVAttributes[DataIndex])
	{
		bool bHasValidUVs = false;
		FVector2D InterpolatedUV = FVector2D::ZeroVector;
		UGeometryScriptLibrary_MeshQueryFunctions::GetInterpolatedTriangleUV(DynamicMeshes[DataIndex], /*UVSetIndex=*/ UVChannel, TriangleId, BarycentricCoord, bHasValidUVs, InterpolatedUV);
		if (bHasValidUVs)
		{
			check(OutputData[DataIndex]);
			OutputData[DataIndex]->Metadata->InitializeOnSet(MetadataEntry);
			UVAttributes[DataIndex]->SetValue(MetadataEntry, InterpolatedUV);
		}
	}

	if (MaterialIdAttributes.IsValidIndex(DataIndex) && MaterialIdAttributes[DataIndex])
	{
		bool bIsValidTriangle = false;
		const int32 MaterialId = UGeometryScriptLibrary_MeshMaterialFunctions::GetTriangleMaterialID(DynamicMeshes[DataIndex], TriangleId, bIsValidTriangle);

		if (bIsValidTriangle)
		{
			check(OutputData[DataIndex]);
			OutputData[DataIndex]->Metadata->InitializeOnSet(MetadataEntry);
			MaterialIdAttributes[DataIndex]->SetValue(MetadataEntry, MaterialId);

			if (MaterialAttributes.IsValidIndex(DataIndex) && MaterialAttributes[DataIndex] && AssetMaterialList.IsValidIndex(DataIndex) && AssetMaterialList[DataIndex].IsValidIndex(MaterialId) && AssetMaterialList[DataIndex][MaterialId])
			{
				MaterialAttributes[DataIndex]->SetValue(MetadataEntry, AssetMaterialList[DataIndex][MaterialId]);
			}
		}
	}

	if (TriangleIdAttributes.IsValidIndex(DataIndex) && TriangleIdAttributes[DataIndex])
	{
		check(OutputData[DataIndex]);
		OutputData[DataIndex]->Metadata->InitializeOnSet(MetadataEntry);
		TriangleIdAttributes[DataIndex]->SetValue(MetadataEntry, TriangleId);
	}
}

void FPCGMeshSamplerContext::SetPointColorAndDensity(SetPointDensityFunc SetPointDensityFuncPtr, int32 TriangleId, const FVector& BarycentricCoord, FPCGPoint& OutPoint, int32 DataIndex, TOptional<FLinearColor> DefaultColor)
{
	FLinearColor Color;
	bool bValidVertexColor = false;
	UGeometryScriptLibrary_MeshQueryFunctions::GetInterpolatedTriangleVertexColor(DynamicMeshes[DataIndex], TriangleId, BarycentricCoord, DefaultColor ? DefaultColor.GetValue() : FLinearColor::White, bValidVertexColor, Color);
	OutPoint.Color = Color;
	SetPointDensityFuncPtr(Color, OutPoint);
}

UPCGMeshSamplerSettings::UPCGMeshSamplerSettings()
{
	if (PCGHelpers::IsNewObjectAndNotDefault(this))
	{
		PointSteepness = 1.0f;
	}
}

void UPCGMeshSamplerSettings::PostLoad()
{
	Super::PostLoad();
	
#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PCGMeshSamplerDefaultColorPCGSubdivideSplineBoundsFixes)
	{
		// For all previous nodes, we'll force this option to false. for retro-compatibility
		bHasDefaultColorOverride = false;
	}
#endif // WITH_EDITOR
}

TArray<FPCGPinProperties> UPCGMeshSamplerSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	if (bExtractMeshFromInput)
	{
		Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Any).SetRequiredPin();
	}

	return Properties;
}

TArray<FPCGPinProperties> UPCGMeshSamplerSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Point, /*bInAllowMultipleConnections=*/bExtractMeshFromInput, /*bAllowMultipleData=*/bExtractMeshFromInput);
	return Properties;
}

FPCGElementPtr UPCGMeshSamplerSettings::CreateElement() const
{
	return MakeShared<FPCGMeshSamplerElement>();
}

#if WITH_EDITOR
FText UPCGMeshSamplerSettings::GetNodeTooltipText() const
{
	return LOCTEXT("MeshSamplerNodeTooltip", "Sample points on a static mesh.");
}

void UPCGMeshSamplerSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (Mesh.IsNull() || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGMeshSamplerSettings, Mesh)))
	{
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Mesh.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(Key).Emplace(this, /*bCulling=*/false);
}
#endif // WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FPCGMeshSamplerContext::~FPCGMeshSamplerContext()
{
	// The context can be destroyed if the task is canceled. In that case, we need to notify the futures that they should stop,
	// and wait for them to finish (as the futures use some data stored in the context that will go dangling if the context is destroyed).
	StopSampling = true;
	for (TFuture<bool>& SamplingFuture : SamplingFutures)
	{
		if (SamplingFuture.IsValid() && !SamplingFuture.IsReady())
		{
			SamplingFuture.Wait();
		}
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FPCGMeshSamplerContext::AddExtraStructReferencedObjects(FReferenceCollector& Collector)
{
	for (TObjectPtr<UDynamicMesh>& DynamicMesh : DynamicMeshes)
	{
		if (DynamicMesh)
		{
			Collector.AddReferencedObject(DynamicMesh);
		}
	}
}

FPCGContext* FPCGMeshSamplerElement::CreateContext()
{
	return new FPCGMeshSamplerContext();
}

bool FPCGMeshSamplerElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMeshSamplerElement::PrepareData);
	FPCGMeshSamplerContext* Context = static_cast<FPCGMeshSamplerContext*>(InContext);

	check(Context);

	if (!Context->ExecutionSource.IsValid())
	{
		return true;
	}

	const UPCGMeshSamplerSettings* Settings = Context->GetInputSettings<UPCGMeshSamplerSettings>();
	check(Settings);

	// 1. Request load for meshes/inputs. Return false if we need to wait, otherwise continue.
	if (!Context->WasLoadRequested())
	{
		const bool bNeedToWait = !Context->InitializeAndRequestLoad(PCGPinConstants::DefaultInputLabel, Settings->InputSource, { Settings->Mesh.ToSoftObjectPath() }, /*bPersistAllData=*/false, /*bSilenceErrorOnEmptyObjectPath=*/true, Settings->bSynchronousLoad);

		if (bNeedToWait)
		{
			return false;
		}
	}

	UGeometryScriptDebug* Debug = FPCGContext::NewObject_AnyThread<UGeometryScriptDebug>(Context);

#if WITH_EDITOR
	FPCGDynamicTrackingHelper DynamicTracking;
	DynamicTracking.EnableAndInitialize(Context);
#endif // WITH_EDITOR

	for (const auto& [Path, DummyIndex, DummyIndex2] : Context->PathsToObjectsAndDataIndex)
	{
		UObject* Object = Path.ResolveObject();
		if (!Object)
		{
			PCGE_LOG(Error, GraphAndLog, FText::Format(LOCTEXT("ObjectDoesNotExist", "Provided object does not exist or could not be loaded: '{0}'"), FText::FromString(Path.ToString())));
			return true;
		}

		// For tracking purposes, keep the original object
		const UObject* OriginalObject = Object;

		// Check if the object is a Blueprint of type Actor, if so, extract the class to be read just after
		if (const UBlueprint* Blueprint = Cast<const UBlueprint>(Object))
		{
			if (UClass* BlueprintClass = Blueprint->GeneratedClass.Get(); BlueprintClass && BlueprintClass->IsChildOf<AActor>())
			{
				Object = BlueprintClass;
			}
		}

		TArray<UObject*> Candidates;

		EGeometryScriptOutcomePins Outcome = EGeometryScriptOutcomePins::Failure;
		Debug->Messages.Reset();

		// Check if the UObject is a class of an actor. If so, find the all scene components
		if (UClass* Class = Cast<UClass>(Object); Class && Class->IsChildOf<AActor>())
		{
			AActor::ForEachComponentOfActorClassDefault<USceneComponent>(Class, [&Candidates](const USceneComponent* Component)
				{ 
					Candidates.Add(const_cast<USceneComponent*>(Component));
					return true; // Continue until we find them all
				});
		}
		else if (AActor* Actor = Cast<AActor>(Object))
		{
			Actor->ForEachComponent<USceneComponent>(/*bIncludeFromChildActor=*/ false, [&Candidates](const USceneComponent* Component)
				{
					Candidates.Add(const_cast<USceneComponent*>(Component));
				});
		}
		else
		{
			Candidates.Add(Object);
		}

		bool bDynMeshAdded = false;

		// @todo_pcg: Better support of multiple candidates. For now, take the first valid one.
		for (UObject* Candidate : Candidates)
		{
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(Candidate))
			{
				// Adaptation of UGeometryScriptLibrary_SceneUtilityFunctions::CopyMeshFromComponent, since we don't have access to the Material list with this one.
				Context->DynamicMeshes.Add(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context));
				bDynMeshAdded = true;
				FTransform Transform;
				FText ErrorMessage;
				UE::Conversion::FToMeshOptions Options{};
				Options.bWantInstanceColors = true;
				Options.LODType = PCGGeometryHelpers::SafeConversionLODType(Settings->RequestedLODType);
				Options.LODIndex = Settings->RequestedLODIndex;

				UE::Geometry::FDynamicMesh3 TempDynMesh{};

				TArray<UMaterialInterface*>* ComponentMaterialListPtr = Settings->bOutputMaterialInfo ? &Context->ComponentMaterialList.Emplace_GetRef() : nullptr;
				TArray<UMaterialInterface*>* AssetMaterialListPtr = Settings->bOutputMaterialInfo ? &Context->AssetMaterialList.Emplace_GetRef() : nullptr;

				const bool bSuccess = UE::Conversion::SceneComponentToDynamicMesh(SceneComponent, Options, /*bTransformToWorld=*/false, TempDynMesh, Transform, ErrorMessage, ComponentMaterialListPtr, AssetMaterialListPtr);
				if (!bSuccess)
				{
					Outcome = EGeometryScriptOutcomePins::Failure;
					FGeometryScriptDebugMessage DebugMessage;
					DebugMessage.Message = std::move(ErrorMessage);
					Debug->Messages.Emplace(std::move(DebugMessage));
				}
				else
				{
					Outcome = EGeometryScriptOutcomePins::Success;
					Context->DynamicMeshes.Last()->SetMesh(std::move(TempDynMesh));

#if WITH_EDITOR
					// Track the scene component if is it different from the original object. The original object is tracked outside of this loop.
					if (OriginalObject != SceneComponent)
					{
						DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(FSoftObjectPath(SceneComponent)), /*bIsCulled=*/false);
					}
#endif // WITH_EDITOR
				}
			}
			else if (Object->IsA<UStaticMesh>() || Object->IsA<USkeletalMesh>())
			{
				const FGeometryScriptMeshReadLOD MeshReadLOD{ Settings->RequestedLODType, Settings->RequestedLODIndex };

				Context->DynamicMeshes.Add(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context));
				bDynMeshAdded = true;
				FGeometryScriptCopyMeshFromAssetOptions Options{};

				if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
				{
					UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromStaticMeshV2(StaticMesh, Context->DynamicMeshes.Last(), Options, MeshReadLOD, Outcome, /*bUseSectionMaterials=*/false, Debug);
					
					if (Outcome == EGeometryScriptOutcomePins::Success && Settings->bOutputMaterialInfo)
					{
						TArray<FName> MaterialSlotNames;
						UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromStaticMesh(StaticMesh, Context->AssetMaterialList.Emplace_GetRef(), MaterialSlotNames, Debug);
					}
				}
				else
				{
					USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(Object);
				
					UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromSkeletalMesh(SkeletalMesh, Context->DynamicMeshes.Last(), Options, MeshReadLOD, Outcome, Debug);
					if (Outcome == EGeometryScriptOutcomePins::Success && Settings->bOutputMaterialInfo)
					{
						TArray<FName> MaterialSlotNames;
						UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromSkeletalMesh(SkeletalMesh, Context->AssetMaterialList.Emplace_GetRef(), MaterialSlotNames, Debug);
					}
				}
			}

			// @todo_pcg: Better support of multiple candidates. For now, take the first valid one.
			if (Outcome == EGeometryScriptOutcomePins::Success)
			{
				break;
			}
		}

		if (Outcome != EGeometryScriptOutcomePins::Success)
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("ObjectToDynamicMeshFailed", "Object to Dynamic mesh failed for object {0}. Either it is not a supported type (StaticMesh/SkeletalMesh/Actor/SceneComponent) or there was a conversion issue."), FText::FromString(Path.ToString())));
			PCGGeometryHelpers::GeometryScriptDebugToPCGLog(Context, Debug);
			if (bDynMeshAdded)
			{
				Context->DynamicMeshes.Last()->MarkAsGarbage();
				Context->DynamicMeshes.RemoveAtSwap(Context->DynamicMeshes.Num() - 1);
			}
		}
		else
		{
#if WITH_EDITOR
			// Track the object, only if the extraction worked
			DynamicTracking.AddToTracking(FPCGSelectionKey::CreateFromPath(Path), /*bIsCulled=*/false);
#endif // WITH_EDITOR
		}
	}

#if WITH_EDITOR
	DynamicTracking.Finalize(Context);
#endif // WITH_EDITOR

	// Manually adding incoming Dynamic meshes data input, copy is necessary if we voxelize, otherwise we'll const cast as it won't be modified (but GeometryScript API is not const friendly).
	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		if (const UPCGDynamicMeshData* InputData = Cast<const UPCGDynamicMeshData>(Input.Data))
		{
			if (Settings->bVoxelize)
			{
				UDynamicMesh* DynMesh = Context->DynamicMeshes.Add_GetRef(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context));
				UE::Geometry::FDynamicMesh3 MeshCopy = InputData->GetDynamicMesh()->GetMeshRef();
				DynMesh->SetMesh(std::move(MeshCopy));
			}
			else
			{
				Context->DynamicMeshes.Emplace(const_cast<UDynamicMesh*>(InputData->GetDynamicMesh()));
			}
		}
	}

	Debug->MarkAsGarbage();

	// Reserve arrays. Add one more entry for the starting indices to have the total number of items to process.
	switch (Settings->SamplingMethod)
	{
	case EPCGMeshSamplingMethod::OnePointPerVertex:
		Context->Positions.SetNum(Context->DynamicMeshes.Num());
		Context->Colors.SetNum(Context->DynamicMeshes.Num());
		Context->Normals.SetNum(Context->DynamicMeshes.Num());
		Context->StartingIndices.SetNumZeroed(Context->DynamicMeshes.Num() + 1);
		break;
	case EPCGMeshSamplingMethod::OnePointPerTriangle:
		Context->TriangleIds.SetNum(Context->DynamicMeshes.Num());
		Context->StartingIndices.SetNumZeroed(Context->DynamicMeshes.Num() + 1);
		break;
	default:
		break;
	}
	
	for (int32 i = 0; i < Context->DynamicMeshes.Num(); ++i)
	{
		UDynamicMesh* DynamicMesh = Context->DynamicMeshes[i];
		if (!DynamicMesh)
		{
			continue;
		}

		if (Settings->bVoxelize)
		{
			FGeometryScriptSolidifyOptions SolidifyOptions{};
			SolidifyOptions.GridParameters.GridCellSize = Settings->VoxelSize;
			SolidifyOptions.GridParameters.SizeMethod = EGeometryScriptGridSizingMethod::GridCellSize;

			UGeometryScriptLibrary_MeshVoxelFunctions::ApplyMeshSolidify(DynamicMesh, SolidifyOptions);

			if (Settings->bRemoveHiddenTriangles)
			{
				FGeometryScriptRemoveHiddenTrianglesOptions RemoveTriangleOptions{};

				UGeometryScriptLibrary_MeshRepairFunctions::RemoveHiddenTriangles(DynamicMesh, RemoveTriangleOptions);
			}
		}

		TArray<FPCGTaggedData>& Outputs = InContext->OutputData.TaggedData;
		UPCGBasePointData* CurrentOutPointData = FPCGContext::NewPointData_AnyThread(Context);
		Context->OutputData.Emplace(CurrentOutPointData);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Context->OutPointData.Emplace(Cast<UPCGPointData>(CurrentOutPointData));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		int32 NumIterations = -1;

		switch (Settings->SamplingMethod)
		{
		case EPCGMeshSamplingMethod::OnePointPerVertex:
		{
			bool bHasValidColors, Dummy, Dummy2;
			UGeometryScriptLibrary_MeshQueryFunctions::GetAllVertexPositions(DynamicMesh, Context->Positions[i], /*bSkipGaps=*/false, /*bHasVertexIDGaps=*/Dummy);
			UGeometryScriptLibrary_MeshVertexColorFunctions::GetMeshPerVertexColors(DynamicMesh, Context->Colors[i], /*bIsValidColorSet=*/bHasValidColors, /*bHasVertexIDGaps=*/Dummy2);
			UGeometryScriptLibrary_MeshNormalsFunctions::GetMeshPerVertexNormals(DynamicMesh, Context->Normals[i], /*bIsValidNormalSet=*/Dummy, /*bHasVertexIDGaps=*/Dummy2);

			if (!bHasValidColors && Settings->bHasDefaultColorOverride)
			{
				for (FLinearColor& Color : *(Context->Colors[i].List))
				{
					Color = Settings->DefaultColorOverride;
				}
			}

			NumIterations = Context->Positions[i].List->Num();
			break;
		}
		case EPCGMeshSamplingMethod::OnePointPerTriangle:
		{
			bool Dummy;
			UGeometryScriptLibrary_MeshQueryFunctions::GetAllTriangleIDs(DynamicMesh, Context->TriangleIds[i], /*bHasTriangleIDGaps=*/Dummy);
			NumIterations = Context->TriangleIds[i].List->Num();
			break;
		}
		case EPCGMeshSamplingMethod::PoissonSampling:
		{
			// No preparation needed
			break;
		}
		default:
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("InvalidOperation", "Invalid sampling operation"), InContext);
			return true;
		}
		}

		if (NumIterations != -1)
		{
			Context->StartingIndices[i + 1] = Context->StartingIndices[i] + NumIterations;
			CurrentOutPointData->SetNumPoints(NumIterations, /*bInitializeValues=*/false);
			// @todo_pcg: could probably be optimzed to save a little memory 
			CurrentOutPointData->AllocateProperties(EPCGPointNativeProperties::All);
		}

		auto CreateAttribute = [this, Context, Metadata = CurrentOutPointData->Metadata]<typename T>(const bool bShouldAdd, const FName AttributeName, T DefaultValue, bool bAllowInterpolation, TArray<FPCGMetadataAttribute<T>*>& OutAttributesArray, const TCHAR* What)
		{
			if (bShouldAdd)
			{
				FPCGMetadataAttribute<T>* Attribute = Metadata->CreateAttribute<T>(AttributeName, std::move(DefaultValue), bAllowInterpolation, /*bOverrideParent=*/true);
				if (!Attribute)
				{
					PCGE_LOG(Warning, GraphAndLog, FText::Format(LOCTEXT("AttributeCreationFailed", "Failed to create attribute {0} for {1}. {1} won't be computed"), FText::FromName(AttributeName), FText::FromString(What)));
				}

				OutAttributesArray.Emplace(Attribute);
			}
		};

		// It's not clear how to compute UVs for Vertices as they are part of multiple triangles. So disable for this mode. Same for triangle ids.
		if (Settings->SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex)
		{
			CreateAttribute(Settings->bExtractUVAsAttribute, Settings->UVAttributeName, FVector2D::ZeroVector, /*bAllowInterpolation=*/true, Context->UVAttributes, TEXT("UVs"));
			CreateAttribute(Settings->bOutputTriangleIds, Settings->TriangleIdAttributeName, -1, /*bAllowInterpolation=*/false, Context->TriangleIdAttributes, TEXT("Triangle IDs"));
			CreateAttribute(Settings->bOutputMaterialInfo, Settings->MaterialIdAttributeName, -1, /*bAllowInterpolation=*/false, Context->MaterialIdAttributes, TEXT("Material IDs"));
			CreateAttribute(Settings->bOutputMaterialInfo, Settings->MaterialAttributeName, FSoftObjectPath(), /*bAllowInterpolation=*/false, Context->MaterialAttributes, TEXT("Material"));
		}

		Outputs.Emplace_GetRef().Data = CurrentOutPointData;
	}

	Context->bDataPrepared = true;
	return true;
}

bool FPCGMeshSamplerElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMeshSamplerElement::Execute);
	FPCGMeshSamplerContext* Context = static_cast<FPCGMeshSamplerContext*>(InContext);

	check(Context);

	if (!Context->bDataPrepared || Context->OutputData.IsEmpty())
	{
		return true;
	}

	const UPCGMeshSamplerSettings* Settings = Context->GetInputSettings<UPCGMeshSamplerSettings>();
	check(Settings);

	bool bIsDone = false;

	// Preparing the set function to extract the color to density.
	auto SetPointDensityTo1 = [](const FLinearColor&, FPCGPoint& OutPoint){ OutPoint.Density = 1.0f; };
	auto SetPointDensityToRed = [](const FLinearColor& Color, FPCGPoint& OutPoint){ OutPoint.Density = Color.R; };
	auto SetPointDensityToGreen = [](const FLinearColor& Color, FPCGPoint& OutPoint){ OutPoint.Density = Color.G; };
	auto SetPointDensityToBlue = [](const FLinearColor& Color, FPCGPoint& OutPoint){ OutPoint.Density = Color.B; };
	auto SetPointDensityToAlpha = [](const FLinearColor& Color, FPCGPoint& OutPoint){ OutPoint.Density = Color.A; };

	// Store it in a function pointer.
	void(*SetPointDensityPtr)(const FLinearColor&, FPCGPoint&) = SetPointDensityTo1;

	if (Settings->bUseColorChannelAsDensity)
	{
		switch (Settings->ColorChannelAsDensity)
		{
		case EPCGColorChannel::Red:
			SetPointDensityPtr = SetPointDensityToRed;
			break;
		case EPCGColorChannel::Green:
			SetPointDensityPtr = SetPointDensityToGreen;
			break;
		case EPCGColorChannel::Blue:
			SetPointDensityPtr = SetPointDensityToBlue;
			break;
		case EPCGColorChannel::Alpha:
			SetPointDensityPtr = SetPointDensityToAlpha;
			break;
		default:
			checkNoEntry();
			break;
		}
	}

	switch (Settings->SamplingMethod)
	{
	case EPCGMeshSamplingMethod::OnePointPerVertex:
	{
		bIsDone = PCGMeshSampler::SampleOnePointPerVertex(Settings, Context, SetPointDensityPtr);
		break;
	}
	case EPCGMeshSamplingMethod::OnePointPerTriangle:
	{
		bIsDone = PCGMeshSampler::SampleOnePointPerTriangle(Settings, Context, SetPointDensityPtr);
		break;
	}
	case EPCGMeshSamplingMethod::PoissonSampling:
	{
		bIsDone = PCGMeshSampler::PoissonSampling(Settings, Context, SetPointDensityPtr);
		break;
	}
	default:
	{
		// Already logged in the PrepareData, we should never arrive there
		checkNoEntry();
		return true;
	}
	}

	return bIsDone;
}

#undef LOCTEXT_NAMESPACE
