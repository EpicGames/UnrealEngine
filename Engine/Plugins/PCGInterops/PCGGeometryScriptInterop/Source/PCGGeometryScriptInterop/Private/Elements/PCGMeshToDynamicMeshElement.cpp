// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGMeshToDynamicMeshElement.h"

#include "PCGContext.h"
#include "PCGModule.h"
#include "Data/PCGDynamicMeshData.h"
#include "Helpers/PCGGeometryHelpers.h"

#if WITH_EDITOR
#include "Helpers/PCGDynamicTrackingHelpers.h"
#endif // WITH_EDITOR

#include "UDynamicMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryScript/MeshAssetFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGMeshToDynamicMeshElement)

#define LOCTEXT_NAMESPACE "PCGMeshToDynamicMeshElement"

#if WITH_EDITOR
FName UPCGMeshToDynamicMeshSettings::GetDefaultNodeName() const
{
	return FName(TEXT("MeshToDynamicMeshElement"));
}

FText UPCGMeshToDynamicMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Mesh To Dynamic Mesh Element");
}

FText UPCGMeshToDynamicMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Convert a static/skeletal mesh into a dynamic mesh data.");
}

void UPCGMeshToDynamicMeshSettings::GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const
{
	if (Mesh.IsNull() || IsPropertyOverriddenByPin(GET_MEMBER_NAME_CHECKED(UPCGMeshToDynamicMeshSettings, Mesh)))
	{
		return;
	}

	FPCGSelectionKey Key = FPCGSelectionKey::CreateFromPath(Mesh.ToSoftObjectPath());

	OutKeysToSettings.FindOrAdd(MoveTemp(Key)).Emplace(this, /*bCulling=*/false);
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGMeshToDynamicMeshSettings::CreateElement() const
{
	return MakeShared<FPCGMeshToDynamicMeshElement>();
}

TArray<FPCGPinProperties> UPCGMeshToDynamicMeshSettings::InputPinProperties() const
{
	return {};
}

TArray<FPCGPinProperties> UPCGMeshToDynamicMeshSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::DynamicMesh, false, false);
	return Properties;
}

bool FPCGMeshToDynamicMeshElement::CanExecuteOnlyOnMainThread(FPCGContext* Context) const
{
	// Without context, we can't know, so force it in the main thread to be safe.
	return !Context || Context->CurrentPhase == EPCGExecutionPhase::PrepareData;
}

FPCGContext* FPCGMeshToDynamicMeshElement::CreateContext()
{
	return new FPCGMeshToDynamicMeshContext();
}

bool FPCGMeshToDynamicMeshElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGStaticMeshToDynamicMeshElement::Execute);

	FPCGMeshToDynamicMeshContext* Context = static_cast<FPCGMeshToDynamicMeshContext*>(InContext);
	check(Context);

	const UPCGMeshToDynamicMeshSettings* Settings = InContext->GetInputSettings<UPCGMeshToDynamicMeshSettings>();
	check(Settings);
	
	if (Context->WasLoadRequested() || Settings->Mesh.IsNull())
	{
		return true;
	}

	TArray<FSoftObjectPath> ObjectsToLoad;
	if (Settings->bExtractMaterials)
	{
		Algo::Transform(Settings->OverrideMaterials, ObjectsToLoad, [](const TSoftObjectPtr<UMaterialInterface>& MaterialSoftPtr) { return MaterialSoftPtr.ToSoftObjectPath(); });
	}

	ObjectsToLoad.Add(Settings->Mesh.ToSoftObjectPath());
	
	return Context->RequestResourceLoad(Context, std::move(ObjectsToLoad), !Settings->bSynchronousLoad);
}

bool FPCGMeshToDynamicMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGMeshToDynamicMeshElement::Execute);

	check(InContext);

	const UPCGMeshToDynamicMeshSettings* Settings = InContext->GetInputSettings<UPCGMeshToDynamicMeshSettings>();
	check(Settings);

	UStaticMesh* StaticMesh = Cast<UStaticMesh>(Settings->Mesh.Get());
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Settings->Mesh.Get());
	
	if (!StaticMesh && !SkeletalMesh)
	{
		if (!Settings->Mesh.IsNull())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MeshNull", "Object {0} failed to load or was not a Static Mesh nor Skeletal Mesh."), FText::FromString(Settings->Mesh.ToSoftObjectPath().ToString())), InContext);
		}
		
		return true;
	}

	check(StaticMesh || SkeletalMesh);

#if WITH_EDITOR
	if (InContext->IsValueOverriden(GET_MEMBER_NAME_CHECKED(UPCGMeshToDynamicMeshSettings, Mesh)))
	{
		FPCGDynamicTrackingHelper::AddSingleDynamicTrackingKey(InContext, FPCGSelectionKey::CreateFromPath(Settings->Mesh.ToSoftObjectPath()), /*bIsCulled=*/false);
	}
#endif // WITH_EDITOR

	if (Settings->bExtractMaterials && !Settings->OverrideMaterials.IsEmpty())
	{
		const int32 NumMeshMaterials = StaticMesh ? StaticMesh->GetStaticMaterials().Num() : SkeletalMesh->GetMaterials().Num();
		if (NumMeshMaterials != Settings->OverrideMaterials.Num())
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("MismatchMaterials", "Mismatch number between Mesh materials ({0}) and override materials ({1})"), NumMeshMaterials, Settings->OverrideMaterials.Num()));
			return true;
		}
		else if (const TSoftObjectPtr<UMaterialInterface>* UnloadedMaterial = Settings->OverrideMaterials.FindByPredicate([](const TSoftObjectPtr<UMaterialInterface>& MaterialSoftPtr) -> bool { return !MaterialSoftPtr.Get(); }))
		{
			PCGLog::LogErrorOnGraph(FText::Format(LOCTEXT("UnloadedMaterial", "Material {0} failed to load."), FText::FromString(UnloadedMaterial->ToSoftObjectPath().ToString())));
			return true;
		}
	}

	UPCGDynamicMeshData* DynMeshData = FPCGContext::NewObject_AnyThread<UPCGDynamicMeshData>(InContext);
	
	const FGeometryScriptMeshReadLOD MeshReadLOD{ Settings->RequestedLODType, Settings->RequestedLODIndex };
	FGeometryScriptCopyMeshFromAssetOptions Options{};

	UGeometryScriptDebug* Debug = FPCGContext::NewObject_AnyThread<UGeometryScriptDebug>(InContext);
	EGeometryScriptOutcomePins Outcome = EGeometryScriptOutcomePins::Success;
	
	if (StaticMesh)
	{
		UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromStaticMeshV2(StaticMesh, DynMeshData->GetMutableDynamicMesh(), Options, MeshReadLOD, Outcome, /*bUseSectionMaterials=*/false, Debug);
	}
	else
	{
		UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshFromSkeletalMesh(SkeletalMesh, DynMeshData->GetMutableDynamicMesh(), Options, MeshReadLOD, Outcome, Debug);
	}

	if (Outcome == EGeometryScriptOutcomePins::Success)
	{
		if (Settings->bExtractMaterials)
		{
			TArray<UMaterialInterface*> Materials;
			
			if (!Settings->OverrideMaterials.IsEmpty())
			{
				Algo::Transform(Settings->OverrideMaterials, Materials, [](const TSoftObjectPtr<UMaterialInterface>& MaterialSoftPtr) { return MaterialSoftPtr.Get(); });
			}
			else if (StaticMesh)
			{
				TArray<FName> MaterialSlotNames;
				UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromStaticMesh(StaticMesh, Materials, MaterialSlotNames);
			}
			else
			{
				TArray<FName> MaterialSlotNames;
				UGeometryScriptLibrary_StaticMeshFunctions::GetMaterialListFromSkeletalMesh(SkeletalMesh, Materials, MaterialSlotNames);
			}

			DynMeshData->SetMaterials(MoveTemp(Materials));
		}

		InContext->OutputData.TaggedData.Emplace_GetRef().Data = DynMeshData;
	}
	else
	{
		PCGGeometryHelpers::GeometryScriptDebugToPCGLog(InContext, Debug);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
