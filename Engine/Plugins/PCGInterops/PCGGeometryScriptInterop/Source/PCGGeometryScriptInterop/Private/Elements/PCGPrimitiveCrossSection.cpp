// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGPrimitiveCrossSection.h"


#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGSplineData.h"
#include "Helpers/PCGGeometryHelpers.h"
#include "Helpers/PCGHelpers.h"

#include "UDynamicMesh.h"
#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Components/SplineComponent.h"
#include "GeometryScript/MeshBooleanFunctions.h"
#include "GeometryScript/MeshComparisonFunctions.h"
#include "GeometryScript/MeshQueryFunctions.h"
#include "GeometryScript/MeshSelectionFunctions.h"
#include "GeometryScript/MeshSelectionQueryFunctions.h"
#include "GeometryScript/MeshSimplifyFunctions.h"
#include "GeometryScript/SceneUtilityFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGPrimitiveCrossSection)

#define LOCTEXT_NAMESPACE "PCGPrimitiveCrossSectionElement"

namespace PCGPrimitiveCrossSection::Constants
{
	const FName DefaultExtrusionVectorAttributeName(TEXT("ExtrusionVector"));
	static constexpr FGeometryScriptMeshPlaneCutOptions CutPlaneOptions{.bFillHoles = false, .bFillSpans = false, .bFlipCutSide = true};
}

struct FCrossSection
{
	int Tier;
	double Height;
	TArray<FVector> PointLocations;
};

UPCGPrimitiveCrossSectionSettings::UPCGPrimitiveCrossSectionSettings()
{
	ExtrusionVectorAttribute.SetAttributeName(PCGPrimitiveCrossSection::Constants::DefaultExtrusionVectorAttributeName);
}

TArray<FPCGPinProperties> UPCGPrimitiveCrossSectionSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultInputLabel, EPCGDataType::Volume | EPCGDataType::Primitive | EPCGDataType::DynamicMesh);
	return PinProperties;
}

TArray<FPCGPinProperties> UPCGPrimitiveCrossSectionSettings::OutputPinProperties() const
{
	TArray<FPCGPinProperties> PinProperties;
	PinProperties.Emplace(PCGPinConstants::DefaultOutputLabel, EPCGDataType::Spline);
	return PinProperties;
}

FPCGElementPtr UPCGPrimitiveCrossSectionSettings::CreateElement() const
{
	return MakeShared<FPCGPrimitiveCrossSectionElement>();
}

bool FPCGPrimitiveCrossSectionElement::ExecuteInternal(FPCGContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGPrimitiveCrossSectionElement::Execute);

	const UPCGPrimitiveCrossSectionSettings* Settings = Context->GetInputSettings<UPCGPrimitiveCrossSectionSettings>();
	const TArray<FPCGTaggedData> PrimitiveInputs = Context->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);

	if (PrimitiveInputs.IsEmpty())
	{
		return true;
	}

	TArray<UDynamicMesh*> DynamicMeshes;
#if WITH_EDITOR
	UGeometryScriptDebug* DynamicMeshDebug = FPCGContext::NewObject_AnyThread<UGeometryScriptDebug>(Context);
#else
	UGeometryScriptDebug* DynamicMeshDebug = nullptr;
#endif // WITH_EDITOR

	// Collect all the primitives and append them to the dynamic mesh
	for (const FPCGTaggedData& TaggedData : PrimitiveInputs)
	{
		PCGGeometryHelpers::ConvertDataToDynMeshes(TaggedData.Data, Context, DynamicMeshes, /*bMergeMeshes=*/false, DynamicMeshDebug);
	}

	if (DynamicMeshes.IsEmpty())
	{
		return true;
	}

	// Dissolve the meshes down until there are none intersecting.
	bool bFoundAnyIntersection;
	do
	{
		bFoundAnyIntersection = false;
		for (int32 FirstMeshIndex = 0; FirstMeshIndex < DynamicMeshes.Num(); ++FirstMeshIndex)
		{
			for (int32 SecondMeshIndex = FirstMeshIndex + 1; SecondMeshIndex < DynamicMeshes.Num(); ++SecondMeshIndex)
			{
				bool bFoundIntersection = false;
				UGeometryScriptLibrary_MeshComparisonFunctions::IsIntersectingMesh(
						DynamicMeshes[FirstMeshIndex],
						FTransform::Identity,
						DynamicMeshes[SecondMeshIndex],
						FTransform::Identity,
						bFoundIntersection,
						DynamicMeshDebug);

				// Found an overlapping primitive to boolean.
				if (bFoundIntersection)
				{
					bFoundAnyIntersection = true;

					UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshBoolean(
						DynamicMeshes[FirstMeshIndex],
						FTransform::Identity,
						DynamicMeshes[SecondMeshIndex],
						FTransform::Identity,
						EGeometryScriptBooleanOperation::Union,
						FGeometryScriptMeshBooleanOptions(), // Default parameters are fine.
						DynamicMeshDebug);

					// Remove the merged mesh and start this index over again, since the array has changed.
					DynamicMeshes.RemoveAtSwap(SecondMeshIndex);
					--FirstMeshIndex;
					break;
				}
			}
		}
	}
	while (bFoundAnyIntersection);

	const FVector SliceDirection = Settings->SliceDirection.GetSafeNormal();

	for (UDynamicMesh* DynamicMesh : DynamicMeshes)
	{
		check(DynamicMesh);

		// Reduce vertex count by simplifying coplanar triangles. Also removes index gaps.
		UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToPlanar(DynamicMesh, FGeometryScriptPlanarSimplifyOptions(), DynamicMeshDebug);

		FGeometryScriptVectorList VertexList;
		bool bHasGapsDummy = false;
		UGeometryScriptLibrary_MeshQueryFunctions::GetAllVertexPositions(DynamicMesh, VertexList, /*bSkipGaps=*/false, bHasGapsDummy);

		if (!VertexList.List.IsValid() ||
			VertexList.List->IsEmpty() ||
			VertexList.List->Num() > Settings->MaxMeshVertexCount) // A safeguard to prevent analysis on a significantly large mesh.
		{
			PCGLog::LogWarningOnGraph(LOCTEXT("MaxMeshVertexCount", "Mesh is invalid, empty, or exceeds Max Mesh Vertex Count and will be skipped."), Context);
			continue;
		}

		FVector SliceOrigin = FVector::ZeroVector;
		TArray<TPair<double, double>> TierHeights;

		// Build tiers based on projection of vertices against the slicing axis
		{
			double MinProjectionScalar = std::numeric_limits<double>::max();

			// Project vertices along the slice direction and find the minimum height
			TArray<double> ProjectionScalars;
			ProjectionScalars.Reserve(VertexList.List->Num());
			for (const FVector& Vertex : *VertexList.List)
			{
				// Implementation note: since the direction vector is guaranteed to be normalized, we can use the projection scalar as distance
				double ProjectionScalar = FVector::DotProduct(Vertex, SliceDirection);
				ProjectionScalars.Emplace(ProjectionScalar);

				// Find the min and max projected "height" to define the slicing extents
				if (MinProjectionScalar > ProjectionScalar)
				{
					MinProjectionScalar = ProjectionScalar;
					// Find the vertex for which we'll start slicing
					SliceOrigin = Vertex;
				}
			}

			// Merge at the user's threshold, or at a minimum threshold to account for numerical or mesh imprecision.
			const double MergingThreshold = Settings->bEnableTierMerging ? Settings->TierMergingThreshold : PCGPrimitiveCrossSection::Constants::MinTierMergingThreshold;
			TierHeights.Reserve(ProjectionScalars.Num() + 1);

			// Once sorted, all similar tier ranges will be consecutive.
			ProjectionScalars.Sort();

			for (const double& Projection : ProjectionScalars)
			{
				double AdjustedProjection = Projection - MinProjectionScalar;
				bool bNeedsNewTier = false;

				// Update previous tier if there is one
				if (!TierHeights.IsEmpty())
				{
					bNeedsNewTier = (AdjustedProjection - TierHeights.Last().Value) >= MergingThreshold;
					TierHeights.Last().Value = AdjustedProjection;
				}
				else
				{
					bNeedsNewTier = true;
				}

				if (bNeedsNewTier)
				{
					TierHeights.Emplace(AdjustedProjection, AdjustedProjection);
				}
			}
		}

		// Set up the slice plane for GeometryScript, represented by an FTransform
		FTransform SlicePlaneTransform;
		SlicePlaneTransform.SetLocation(SliceOrigin);
		FQuat AdjustedRotation = FQuat::FindBetween(FVector::UpVector, SliceDirection);
		SlicePlaneTransform.SetRotation(AdjustedRotation);

		TArray<FCrossSection> CrossSections;
		for (int TierIndex = 0; TierIndex < TierHeights.Num(); ++TierIndex)
		{
			FVector SliceLocation = SliceOrigin + SliceDirection * TierHeights[TierIndex].Key;
			SlicePlaneTransform.SetLocation(SliceLocation);

			// Cuts the mesh at the specified plane, leaving a hole
			UGeometryScriptLibrary_MeshBooleanFunctions::ApplyMeshPlaneCut(DynamicMesh, SlicePlaneTransform, PCGPrimitiveCrossSection::Constants::CutPlaneOptions, DynamicMeshDebug);

			// Break now as there is no point in continuing along this invalid mesh
			if (DynamicMesh->GetTriangleCount() == 0)
			{
				break;
			}

			// The cut algorithm does not result in simplified planes, so simplify it
			UGeometryScriptLibrary_MeshSimplifyFunctions::ApplySimplifyToPlanar(DynamicMesh, FGeometryScriptPlanarSimplifyOptions(), DynamicMeshDebug); // TODO: verify defaults

			// TODO: Investigate if there's a simpler API call to get the cut plane triangles (vertices)
			// Since select box is AABB, in order to select a slice, select with a plane just before and subtract the one just after
			FGeometryScriptMeshSelection CurrentSelection{};
			UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsWithPlane(
				DynamicMesh,
				CurrentSelection,
				SliceLocation - (SliceDirection * UE_DOUBLE_KINDA_SMALL_NUMBER), // Move back a bit
				SliceDirection,
				EGeometryScriptMeshSelectionType::Vertices);

			FGeometryScriptMeshSelection ExclusiveSelection{};
			UGeometryScriptLibrary_MeshSelectionFunctions::SelectMeshElementsWithPlane(
				DynamicMesh,
				ExclusiveSelection,
				SliceLocation + (SliceDirection * UE_DOUBLE_KINDA_SMALL_NUMBER), // Move forward a bit
				SliceDirection,
				EGeometryScriptMeshSelectionType::Vertices);

			UGeometryScriptLibrary_MeshSelectionFunctions::CombineMeshSelections(
				CurrentSelection,
				ExclusiveSelection,
				CurrentSelection,
				EGeometryScriptCombineSelectionMode::Subtract);

			TArray<FGeometryScriptIndexList> VertexIndexLists;
			TArray<FGeometryScriptPolyPath> PolyPaths;
			int NumLoops;
			bool bFoundErrors;
			// TODO: Find a more direct way to determine multiple poly paths--one for each island--and order the vertices
			// Implementation note: Using Boundary Loops to create multiple 2D poly paths and order points.
			UGeometryScriptLibrary_MeshSelectionQueryFunctions::GetMeshSelectionBoundaryLoops(
				DynamicMesh,
				CurrentSelection,
				VertexIndexLists,
				PolyPaths,
				NumLoops,
				bFoundErrors,
				DynamicMeshDebug);

#if WITH_EDITOR
			// At this point, we're done with GeometryScript for this iteration. Print errors if they occur.
			PCGGeometryHelpers::GeometryScriptDebugToPCGLog(Context, DynamicMeshDebug);
#endif // WITH_EDITOR

			if (NumLoops < 1 || PolyPaths.IsEmpty())
			{
				continue;
			}

			const FQuat AdjustedRotationInverse = AdjustedRotation.Inverse();

			// Shoelace formula - https://en.wikipedia.org/wiki/Shoelace_formula.
			// Copy PointLocations in by value to be used as a temporary container.
			auto ComputePolyPathArea = [&AdjustedRotationInverse](TArray<FVector>& PointLocations)
			{
				const int NumPoints = PointLocations.Num();
				if (NumPoints < 3)
				{
					return 0.0;
				}

				// Transform the points to 2D referential in order to calculate the surface area
				Algo::ForEach(PointLocations, [&AdjustedRotationInverse](FVector& Location) { Location = AdjustedRotationInverse.RotateVector(Location); });

				double Area = 0.0;
				for (int PointIndex = 0; PointIndex < PointLocations.Num() - 1; ++PointIndex)
				{
					Area += PointLocations[PointIndex].X * PointLocations[PointIndex + 1].Y - PointLocations[PointIndex + 1].X * PointLocations[PointIndex].Y;
				}

				// Calculate the last term (last and first point) separately
				Area += PointLocations[NumPoints - 1].X * PointLocations[0].Y - PointLocations[0].X * PointLocations[NumPoints - 1].Y;

				// The sign of the area could be used to determine winding, if needed in the future
				return FMath::Abs(Area) * 0.5;
			};

			for (int LoopIndex = 0; LoopIndex < NumLoops; ++LoopIndex)
			{
				if (!ensure(PolyPaths[LoopIndex].bClosedLoop))
				{
					continue;
				}

				// Do a planar check of one of the vertices and eliminate poly paths outside our cut plane
				// TODO: This is a crutch to keep the boundary loops from creeping.
				double DotProduct = FMath::Abs(FVector::DotProduct(PolyPaths[LoopIndex].Path->Last() - SliceLocation, SliceDirection));
				if (DotProduct > UE_DOUBLE_KINDA_SMALL_NUMBER)
				{
					continue;
				}

				if (Settings->bEnableMinAreaCulling && ComputePolyPathArea(*PolyPaths[LoopIndex].Path) <= Settings->MinAreaCullingThreshold * 100.0)
				{
					continue;
				}

				const double Height = (TierHeights[TierIndex].Value - TierHeights[TierIndex].Key);

				if (Settings->bEnableMinHeightCulling && Height <= Settings->MinHeightCullingThreshold)
				{
					continue;
				}

				FCrossSection& CurrentTier = CrossSections.Emplace_GetRef();
				CurrentTier.Tier = TierIndex;
				
				const double RoundedHeight = FMath::RoundToDouble(Height);
				// To account for rounding errors and mesh operations.
				CurrentTier.Height = FMath::IsNearlyEqual(Height, RoundedHeight) ? RoundedHeight : Height;
				CurrentTier.PointLocations.Append(*PolyPaths[LoopIndex].Path);
			}
		}

		// Filter cross-sections that would otherwise project to the previous one.
		if (Settings->bRemoveRedundantSections)
		{
			TArray<int, TInlineAllocator<16>> RedundantSections;
			// Implementation note: the cross-sections are sorted by height here, so the highest can be taken
			CrossSections.Sort([](const FCrossSection& LHS, const FCrossSection& RHS) { return LHS.Tier < RHS.Tier; });

			for (int FirstIndex = 0; FirstIndex < CrossSections.Num(); ++FirstIndex)
			{
				if (RedundantSections.Contains(FirstIndex))
				{
					continue;
				}

				for (int SecondIndex = CrossSections.Num() - 1; SecondIndex > FirstIndex; --SecondIndex)
				{
					if (RedundantSections.Contains(SecondIndex))
					{
						continue;
					}

					FCrossSection& FirstSection = CrossSections[FirstIndex];
					const FCrossSection& SecondSection = CrossSections[SecondIndex];

					if (SecondSection.Tier < FirstSection.Tier)
					{
						break;
					}

					// Disqualify based on point count
					if (FirstSection.PointLocations.Num() != SecondSection.PointLocations.Num())
					{
						continue;
					}

					// Check if the cross-section's vertices are all equal to find if it's redundant for culling.
					// Implementation Note: This behavior is not always intuitive with any mesh and will cull even if there are tiers in between.
					// TODO: Some future options might include checking for intermediate tiers between, etc. However, we need to be mindful of merge order.
					bool bCrossSectionIsEqual = true;
					for (int PointIndex = 0; PointIndex < FirstSection.PointLocations.Num(); ++PointIndex)
					{
						auto CompareXYPredicate = [FirstLocation = FirstSection.PointLocations[PointIndex]](const FVector& SecondLocation)
						{
							return FMath::IsNearlyEqual(FirstLocation.X, SecondLocation.X) && FMath::IsNearlyEqual(FirstLocation.Y, SecondLocation.Y);
						};

						if (!Algo::FindByPredicate(SecondSection.PointLocations, CompareXYPredicate))
						{
							bCrossSectionIsEqual = false;
							break;
						}
					}

					if (bCrossSectionIsEqual)
					{
						RedundantSections.Add(SecondIndex);
						// Combine the heights of both sections
						FirstSection.Height += SecondSection.Height;
					}
				}
			}

			// Sort inverse, because they will be remove swapped from the actual array
			RedundantSections.Sort([](const int LHS, const int RHS) { return LHS > RHS; });

			for (int Index : RedundantSections)
			{
				CrossSections.RemoveAtSwap(Index);
			}
		}

		bool bMetadataDomainWarned = false;
		bool bAttributeFailedWarned = false;

		TArray<FSplinePoint> SplinePoints;
		// Create the splines from the tiers
		for (const FCrossSection& CrossSection : CrossSections)
		{
			SplinePoints.Reset(CrossSection.PointLocations.Num());
			FPCGTaggedData& OutputData = Context->OutputData.TaggedData.Emplace_GetRef();

			for (int Index = 0; Index < CrossSection.PointLocations.Num(); ++Index)
			{
				SplinePoints.Emplace(static_cast<float>(Index),
					CrossSection.PointLocations[Index],
					FVector::ZeroVector,
					FVector::ZeroVector,
					FRotator::ZeroRotator,
					FVector::OneVector,
					ESplinePointType::Linear);
			}

			UPCGSplineData* OutSplineData = FPCGContext::NewObject_AnyThread<UPCGSplineData>(Context);
			OutSplineData->Initialize(SplinePoints, true, FTransform::Identity);
			OutputData.Data = OutSplineData;

			UPCGMetadata* Metadata = OutSplineData->MutableMetadata();
			check(Metadata);

			const FPCGMetadataDomainID MetadataDomainID = OutSplineData->GetMetadataDomainIDFromSelector(Settings->ExtrusionVectorAttribute);
			FPCGMetadataDomain* MetadataDomain = Metadata->GetMetadataDomain(MetadataDomainID);

			const FName AttributeName = Settings->ExtrusionVectorAttribute.GetName();

			if (MetadataDomain)
			{
				const FPCGMetadataAttributeBase* Attribute = MetadataDomain->CreateAttribute<FVector>(AttributeName, SliceDirection * CrossSection.Height, /*bAllowsInterpolation=*/false, /*bOverridesParent=*/true);
				
				if (Attribute == nullptr && !bAttributeFailedWarned)
				{
					bAttributeFailedWarned = true;
					PCGLog::Metadata::LogFailToCreateAttributeError<FVector>(AttributeName, Context);
				}
			}
			else if (!bMetadataDomainWarned)
			{
				bMetadataDomainWarned = true;
				PCGLog::Metadata::LogInvalidMetadataDomain(Settings->ExtrusionVectorAttribute, Context);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
