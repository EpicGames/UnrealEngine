// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSaveDynamicMeshToAsset.h"

#include "PCGAssetExporterUtils.h"
#include "PCGContext.h"
#include "PCGParamData.h"
#include "Data/PCGDynamicMeshData.h"
#include "Utils/PCGLogErrors.h"

#include "UDynamicMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryScript/MeshAssetFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSaveDynamicMeshToAsset)

#define LOCTEXT_NAMESPACE "PCGSaveDynamicMeshToAssetElement"

#if WITH_EDITOR
FName UPCGSaveDynamicMeshToAssetSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SaveDynamicMeshToAsset"));
}

FText UPCGSaveDynamicMeshToAssetSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Save Dynamic Mesh To Asset");
}

FText UPCGSaveDynamicMeshToAssetSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Saves dynamic mesh data into a static mesh asset.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSaveDynamicMeshToAssetSettings::CreateElement() const
{
	return MakeShared<FPCGSaveDynamicMeshToAssetElement>();
}

TArray<FPCGPinProperties> UPCGSaveDynamicMeshToAssetSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::DynamicMesh, false, false).SetRequiredPin();
	return Properties;
}

TArray<FPCGPinProperties> UPCGSaveDynamicMeshToAssetSettings::OutputPinProperties() const
{ 
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace(TEXT("OutAssetPath"), EPCGDataType::Param, false, false);
	return Properties;
}

bool FPCGSaveDynamicMeshToAssetElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSaveDynamicMeshToAssetElement::Execute);

	check(InContext);

	// For now we only support writing to an asset in editor build.
#if WITH_EDITOR
	const UPCGSaveDynamicMeshToAssetSettings* Settings = InContext->GetInputSettings<UPCGSaveDynamicMeshToAssetSettings>();
	check(Settings);

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	if (Inputs.IsEmpty())
	{
		return true;
	}
	
	if (Inputs.Num() > 1)
	{
		PCGLog::InputOutput::LogFirstInputOnlyWarning(PCGPinConstants::DefaultInputLabel, InContext);	
	}

	const UPCGDynamicMeshData* DynamicMeshData = Cast<const UPCGDynamicMeshData>(Inputs[0].Data);
	if (!DynamicMeshData || !DynamicMeshData->GetDynamicMesh())
	{
		PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::DynamicMesh, PCGPinConstants::DefaultInputLabel, InContext);
		return true;
	}

	UObject* OutObject = nullptr;
	
	UPackage* OutPackage = UPCGAssetExporterUtils::CreateAsset<UStaticMesh>(Settings->ExportParams, [&DynamicMeshData, InContext, &OutObject, Settings](const FString&, UObject* Asset)
	{
		// Export options, make sure we don't emit a transaction.
		FGeometryScriptCopyMeshToAssetOptions AssetOptions = Settings->CopyMeshToAssetOptions;
		AssetOptions.bEmitTransaction = false;
		AssetOptions.bDeferMeshPostEditChange = false;

		// Force the option to replace materials if the dynamic mesh data has materials
		if (Settings->bExportMaterialsFromDynamicMesh)
		{
			AssetOptions.bReplaceMaterials = !DynamicMeshData->GetMaterials().IsEmpty();
			if (!DynamicMeshData->GetMaterials().IsEmpty())
			{
				AssetOptions.NewMaterials = DynamicMeshData->GetMaterials();
			}
		}

		// Convert merged mesh to static mesh, const_cast because GeometryScript API is not nice with constness, but it will not be modified.
		EGeometryScriptOutcomePins OutResult;
		UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToStaticMesh(const_cast<UDynamicMesh*>(DynamicMeshData->GetDynamicMesh()), CastChecked<UStaticMesh>(Asset), AssetOptions, Settings->MeshWriteLOD, OutResult);

		if (OutResult == EGeometryScriptOutcomePins::Failure)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("ErrorConversion", "Error while converting dynamic mesh to static mesh."), InContext);
			return false;
		}
		else
		{
			OutObject = Asset;
			return true;
		}
	}, InContext);

	if (!OutPackage || !OutObject)
	{
		return true;
	}

	UPCGParamData* OutParamData = NewObject<UPCGParamData>();
	OutParamData->Metadata->CreateAttribute<FSoftObjectPath>(TEXT("AssetPath"), FSoftObjectPath(OutObject), false, false);
	OutParamData->Metadata->AddEntry();

	InContext->OutputData.TaggedData.Emplace_GetRef(Inputs[0]).Data = OutParamData;
#else
	PCGLog::LogWarningOnGraph(LOCTEXT("CannotExportInNonEditor", "Can't save a dynamic mesh to asset in non editor build."), InContext);
#endif // WITH_EDITOR
	
	return true;
}

#undef LOCTEXT_NAMESPACE
