// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSetMaterialsDynamicMesh.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"

#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSetMaterialsDynamicMesh)

#define LOCTEXT_NAMESPACE "PCGSetMaterialsDynamicMesh"

#if WITH_EDITOR
FName UPCGSetMaterialsDynamicMeshSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SetMaterialsDynamicMesh"));
}

FText UPCGSetMaterialsDynamicMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Set Materials On Dynamic Mesh");
}

FText UPCGSetMaterialsDynamicMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Replace the array of materials on the dynamic mesh data.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSetMaterialsDynamicMeshSettings::CreateElement() const
{
	return MakeShared<FPCGSetMaterialsDynamicMeshElement>();
}

bool FPCGSetMaterialsDynamicMeshElement::PrepareDataInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSetMaterialsDynamicMeshElement::Execute);

	check(InContext);

	const UPCGSetMaterialsDynamicMeshSettings* Settings = InContext->GetInputSettings<UPCGSetMaterialsDynamicMeshSettings>();
	check(Settings);

	FPCGSetMaterialsDynamicMeshContext* ThisContext = reinterpret_cast<FPCGSetMaterialsDynamicMeshContext*>(InContext);

	TArray<FSoftObjectPath> ResourcesToLoad;
	ResourcesToLoad.Reserve(Settings->Materials.Num());
	Algo::TransformIf(
		Settings->Materials,
		ResourcesToLoad,
		[](const TSoftObjectPtr<UMaterialInterface>& Material) { return !Material.IsNull(); },
		[](const TSoftObjectPtr<UMaterialInterface>& Material) { return Material.ToSoftObjectPath(); }
	);
	
	return ThisContext->RequestResourceLoad(InContext, MoveTemp(ResourcesToLoad));
}

bool FPCGSetMaterialsDynamicMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSetMaterialsDynamicMeshElement::Execute);

	check(InContext);

	const UPCGSetMaterialsDynamicMeshSettings* Settings = InContext->GetInputSettings<UPCGSetMaterialsDynamicMeshSettings>();
	check(Settings);

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		UPCGDynamicMeshData* OutputData = CopyOrSteal(Input, InContext);
		if (!OutputData)
		{
			continue;
		}

		TArray<UMaterialInterface*> Materials;
		Materials.Reserve(Settings->Materials.Num());
		Algo::Transform(Settings->Materials, Materials, [](const TSoftObjectPtr<UMaterialInterface>& Material) { return Material.Get(); });

		OutputData->SetMaterials(MoveTemp(Materials));
		InContext->OutputData.TaggedData.Emplace_GetRef(Input).Data = OutputData;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
