// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSpawnDynamicMesh.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Components/DynamicMeshComponent.h"
#include "Data/PCGDynamicMeshData.h"
#include "Resources/PCGDynamicMeshManagedComponent.h"

#include "UDynamicMesh.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSpawnDynamicMesh)

#define LOCTEXT_NAMESPACE "PCGSpawnDynamicMeshElement"

#if WITH_EDITOR
FName UPCGSpawnDynamicMeshSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SpawnDynamicMesh"));
}

FText UPCGSpawnDynamicMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Spawn Dynamic Mesh");
}

FText UPCGSpawnDynamicMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Spawn a dynamic mesh component for each dynamic mesh data in input.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSpawnDynamicMeshSettings::CreateElement() const
{
	return MakeShared<FPCGSpawnDynamicMeshElement>();
}

bool FPCGSpawnDynamicMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSpawnDynamicMeshElement::Execute);

	check(InContext);

	const UPCGSpawnDynamicMeshSettings* Settings = InContext->GetInputSettings<UPCGSpawnDynamicMeshSettings>();
	check(Settings);

	AActor* TargetActor = Settings->TargetActor.IsValid() ? Settings->TargetActor.Get() : InContext->GetTypedExecutionTarget<AActor>();
	if (!TargetActor)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("InvalidTargetActor", "Invalid target actor."), InContext);
		return true;
	}
	
	UPCGComponent* SourcePCGComponent = Cast<UPCGComponent>(InContext->ExecutionSource.Get());
	if (!SourcePCGComponent)
	{
		return true;
	}

	for (const FPCGTaggedData& Input : InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel))
	{
		const UPCGDynamicMeshData* DynMeshData = Cast<UPCGDynamicMeshData>(Input.Data);
		if (!DynMeshData)
		{
			PCGLog::InputOutput::LogInvalidInputDataError(InContext);
			continue;
		}

		UPCGDynamicMeshManagedComponent* ManagedComponent = PCGDynamicMeshManagedComponent::GetOrCreateDynamicMeshManagedComponent(InContext, Settings, DynMeshData, TargetActor);
		UDynamicMeshComponent* Component = ManagedComponent ? ManagedComponent->GetComponent() : nullptr;
		if (!Component)
		{
			continue;
		}

		SourcePCGComponent->IgnoreChangeOriginDuringGenerationWithScope(Component, [&DynMeshData, &Component]() { DynMeshData->InitializeDynamicMeshComponentFromData(Component); });

		for (const FString& Tag : Input.Tags)
		{
			Component->ComponentTags.AddUnique(*Tag);
		}

		InContext->OutputData.TaggedData.Emplace(Input);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
