// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGSplineToMesh.h"

#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/PCGSplineData.h"
#include "Data/PCGPolygon2DData.h"

#include "CurveOps/TriangulateCurvesOp.h"
#include "Operations/ExtrudeMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGSplineToMesh)

#define LOCTEXT_NAMESPACE "PCGSplineToMeshElement"

namespace PCGSplineToMeshConstants
{
	const FLazyName MeshTransform = "MeshTransform";
}

#if WITH_EDITOR
FName UPCGSplineToMeshSettings::GetDefaultNodeName() const
{
	return FName(TEXT("SplineToMesh"));
}

FText UPCGSplineToMeshSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Spline To Mesh");
}

FText UPCGSplineToMeshSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Converts a spline into a closed mesh, and optionally extrudes it according to a direction. Note that the operation will 'close' all splines.");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGSplineToMeshSettings::CreateElement() const
{
	return MakeShared<FPCGSplineToMeshElement>();
}

TArray<FPCGPinProperties> UPCGSplineToMeshSettings::InputPinProperties() const
{
	TArray<FPCGPinProperties> Properties;
	Properties.Emplace_GetRef(PCGPinConstants::DefaultInputLabel, EPCGDataType::Spline | EPCGDataType::Polygon2D).SetRequiredPin();
	return Properties;
}

bool FPCGSplineToMeshElement::ExecuteInternal(FPCGContext* InContext) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPCGSplineToMeshElement::Execute);

	check(InContext);

	const UPCGSplineToMeshSettings* Settings = InContext->GetInputSettings<UPCGSplineToMeshSettings>();
	check(Settings);

	const bool bUseDefaultExtrusion = (Settings->ExtrusionVector - FVector::ZAxisVector).IsNearlyZero(UE_SMALL_NUMBER) && Settings->bExtrudeInLocalSpace;

	const TArray<FPCGTaggedData> Inputs = InContext->InputData.GetInputsByPin(PCGPinConstants::DefaultInputLabel);
	for (const FPCGTaggedData& Input : Inputs)
	{
		const UPCGSplineData* InputSplineData = Cast<UPCGSplineData>(Input.Data);
		const UPCGPolygon2DData* InputPolygonData = Cast<UPCGPolygon2DData>(Input.Data);

		if (!InputSplineData && !InputPolygonData)
		{
			PCGLog::InputOutput::LogTypedDataNotFoundWarning(EPCGDataType::Spline | EPCGDataType::Polygon2D, PCGPinConstants::DefaultInputLabel, InContext);
			continue;
		}

		UE::Geometry::FTriangulateCurvesOp TriangulateCurvesOp;
		TriangulateCurvesOp.Thickness = bUseDefaultExtrusion ? Settings->Thickness : 0.0;
		TriangulateCurvesOp.bFlipResult = Settings->bFlipResult;
		TriangulateCurvesOp.FlattenMethod = Settings->FlattenMethod;
		TriangulateCurvesOp.CurveOffset = Settings->CurveOffset;
		TriangulateCurvesOp.OffsetOpenMethod = Settings->OpenCurves;
		TriangulateCurvesOp.OffsetJoinMethod = Settings->JoinMethod;
		TriangulateCurvesOp.OpenEndShape = Settings->EndShapes;
		TriangulateCurvesOp.MiterLimit = Settings->MiterLimit;
		TriangulateCurvesOp.bFlipResult = Settings->bFlipResult;
		
		if (Settings->CurveOffset == 0.0)
		{
			TriangulateCurvesOp.OffsetClosedMethod = EOffsetClosedCurvesMethod::DoNotOffset;
		}
		else
		{
			TriangulateCurvesOp.OffsetClosedMethod = Settings->OffsetClosedCurves;
		}
		
		if (InputSplineData)
		{
			TArray<FVector> SplinePoints;
			InputSplineData->SplineStruct.ConvertSplineToPolyLine(ESplineCoordinateSpace::World, Settings->ErrorTolerance, SplinePoints);
			TriangulateCurvesOp.AddWorldCurve(SplinePoints, InputSplineData->IsClosed(), InputSplineData->SplineStruct.Transform);
		}
		else
		{
			check(InputPolygonData);
			const UE::Geometry::FGeneralPolygon2d& Polygon = InputPolygonData->GetPolygon();
			const FTransform Transform = InputPolygonData->GetTransform();

			TArray<FVector> PolyVertices;
			auto GetVerticesFromPolygon =  [&Transform](const UE::Geometry::FPolygon2d& Poly, TArray<FVector>& PolyVertices)
			{
				const TArray<FVector2D>& PolyVertices2D = Poly.GetVertices();

				PolyVertices.Reset();
				PolyVertices.Reserve(PolyVertices2D.Num());
				Algo::Transform(PolyVertices2D, PolyVertices, [&Transform](const FVector2D& Vtx2D) { return Transform.TransformPosition(FVector(Vtx2D, 0.0)); });
			};

			GetVerticesFromPolygon(Polygon.GetOuter(), PolyVertices);
			TriangulateCurvesOp.AddWorldCurve(PolyVertices, /*bClosed=*/true, Transform);

			for (const UE::Geometry::FPolygon2d& Hole : Polygon.GetHoles())
			{
				GetVerticesFromPolygon(Hole, PolyVertices);
				TriangulateCurvesOp.AddWorldCurve(PolyVertices, /*bClosed=*/true, Transform);
			}

			// Implementation note:
			// It is imperative that the curves are flattened, otherwise the difference will not happen.
			TriangulateCurvesOp.CombineMethod = ECombineCurvesMethod::Difference;
			TriangulateCurvesOp.FlattenMethod = EFlattenCurveMethod::ToBestFitPlane;
		}

		TriangulateCurvesOp.CalculateResult(nullptr);

		TUniquePtr<UE::Geometry::FDynamicMesh3>	DynamicMesh = TriangulateCurvesOp.ExtractResult();

		if (!DynamicMesh || DynamicMesh->TriangleCount() == 0)
		{
			PCGLog::LogErrorOnGraph(LOCTEXT("TriangulationFailed", "Triangulation failed"), InContext);
			continue;
		}

		if (!bUseDefaultExtrusion)
		{
			UE::Geometry::FExtrudeMesh ExtrudeMesh(DynamicMesh.Get());
			ExtrudeMesh.ExtrudedPositionFunc = [Settings](const FVector3d& P, const FVector3f& N, int VID) -> FVector3d 
			{ 
				if (Settings->bExtrudeInLocalSpace)
				{
					FQuat RelativeRotation = FQuat::FindBetweenNormals(FVector::ZAxisVector, FVector(N));
					return P + RelativeRotation.RotateVector(Settings->ExtrusionVector) * Settings->Thickness;
				}
				else
				{
					return P + Settings->ExtrusionVector * Settings->Thickness;
				}
			};
			ExtrudeMesh.Apply();
		}
		
		UE::Geometry::FDynamicMesh3* DynamicMeshPtr = DynamicMesh.Release();
		UPCGDynamicMeshData* DynamicMeshData = FPCGContext::NewObject_AnyThread<UPCGDynamicMeshData>(InContext);
		DynamicMeshData->Initialize(std::move(*DynamicMeshPtr));
		delete DynamicMeshPtr;
		DynamicMeshPtr = nullptr;

		UPCGMetadata* Metadata = DynamicMeshData->MutableMetadata();
		if (FPCGMetadataDomain* DataDomain = Metadata->GetMetadataDomain(PCGMetadataDomainID::Data))
		{
			FPCGMetadataAttribute<FTransform>* TransformAttribute = DataDomain->CreateAttribute<FTransform>(PCGSplineToMeshConstants::MeshTransform, FTransform::Identity, true, true);
			check(TransformAttribute);
			PCGMetadataEntryKey DefaultDomainKey = DataDomain->AddEntry();
			TransformAttribute->SetValue(DefaultDomainKey, TriangulateCurvesOp.GetResultTransform());
		}

		InContext->OutputData.TaggedData.Emplace_GetRef(Input).Data = DynamicMeshData;
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE
