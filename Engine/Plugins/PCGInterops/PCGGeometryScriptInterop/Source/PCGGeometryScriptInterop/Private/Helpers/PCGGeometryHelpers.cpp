// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/PCGGeometryHelpers.h"

#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "Data/PCGCollisionShapeData.h"
#include "Data/PCGCollisionWrapperData.h"
#include "Data/PCGDynamicMeshData.h"
#include "Data/PCGPrimitiveData.h"
#include "Data/PCGVolumeData.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Utils/PCGLogErrors.h"

#include "UDynamicMesh.h"
#include "TransformSequence.h"
#include "Components/BrushComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "GameFramework/Volume.h"
#include "Generators/CapsuleGenerator.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "Generators/SphereGenerator.h"
#include "GeometryScript/SceneUtilityFunctions.h"
#include "Physics/ComponentCollisionUtil.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"

namespace PCGGeometryHelpers
{
	namespace Constants
	{
		static constexpr FGeometryScriptCopyMeshFromComponentOptions CopyMeshFromComponentOptions{ .bWantNormals = false, .bWantTangents = false };
		static constexpr int32 SphereNumTheta = 25;
		static constexpr int32 SphereNumPhi = 25;
		static constexpr int32 CapsuleNumHemisphereSteps = 12;
		static constexpr int32 CapsuleNumCircleSteps = 12;
	}

	void GeometryScriptDebugToPCGLog(FPCGContext* Context, const UGeometryScriptDebug* Debug)
	{
		check(Context && Debug);

		for (const FGeometryScriptDebugMessage& Message : Debug->Messages)
		{
			if (Message.MessageType == EGeometryScriptDebugMessageType::ErrorMessage)
			{
				PCGLog::LogErrorOnGraph(Message.Message, Context);
			}
			else
			{
				PCGLog::LogWarningOnGraph(Message.Message, Context);
			}
		}
	}

	UE::Conversion::EMeshLODType SafeConversionLODType(const EGeometryScriptLODType LODType)
	{
		// Make sure the LOD is valid, and make the conversion.
		UE::Conversion::EMeshLODType RequestedLODType{};

		switch (LODType)
		{
		case EGeometryScriptLODType::MaxAvailable:
			RequestedLODType = UE::Conversion::EMeshLODType::MaxAvailable;
			break;
		case EGeometryScriptLODType::HiResSourceModel:
			RequestedLODType = UE::Conversion::EMeshLODType::HiResSourceModel;
			break;
		case EGeometryScriptLODType::SourceModel:
			RequestedLODType = UE::Conversion::EMeshLODType::SourceModel;
			break;
		case EGeometryScriptLODType::RenderData:
			RequestedLODType = UE::Conversion::EMeshLODType::RenderData;
			break;
		default:
			break;
		}

		return RequestedLODType;
	}

	void RemapMaterials(UE::Geometry::FDynamicMesh3& InMesh, const TArray<UMaterialInterface*>& FromMaterials, TArray<TObjectPtr<UMaterialInterface>>& ToMaterials, const UE::Geometry::FMeshIndexMappings* OptionalMappings)
	{
		if (FromMaterials.IsEmpty() || !InMesh.HasAttributes() || !InMesh.Attributes()->HasMaterialID())
		{
			return;
		}

		// If we have no materials in output, we can just copy the in materials
		if (ToMaterials.IsEmpty())
		{
			ToMaterials = FromMaterials;
			return;
		}

		TMap<int32, int32> MaterialIDRemap;
		MaterialIDRemap.Reserve(FromMaterials.Num());
		
		for (int32 FromMaterialIndex = 0; FromMaterialIndex < FromMaterials.Num(); ++FromMaterialIndex)
		{
			UMaterialInterface* FromMaterial = FromMaterials[FromMaterialIndex];
			int32 ToMaterialIndex = ToMaterials.IndexOfByKey(FromMaterial);
			if (ToMaterialIndex == INDEX_NONE)
			{
				ToMaterialIndex = ToMaterials.Add(FromMaterial);
			}

			if (ToMaterialIndex != FromMaterialIndex)
			{
				MaterialIDRemap.Emplace(FromMaterialIndex, ToMaterialIndex);
			}
		}

		if (!MaterialIDRemap.IsEmpty())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PCGGeometryHelpers::RemapMaterials);
			UE::Geometry::FDynamicMeshMaterialAttribute* MaterialAttribute = InMesh.Attributes()->GetMaterialID();

			auto Remap = [&MaterialAttribute, &MaterialIDRemap](const int32 TriangleID)
			{
				const int32 OriginalMaterialID = MaterialAttribute->GetValue(TriangleID);
				if (const int32* RemappedMaterialID = MaterialIDRemap.Find(OriginalMaterialID))
				{
					MaterialAttribute->SetValue(TriangleID, *RemappedMaterialID);
				}
			};
			
			// TODO: Could be parallelized
			if (OptionalMappings)
			{
				for (const TPair<int32, int32>& MapTriangleID : OptionalMappings->GetTriangleMap().GetForwardMap())
				{
					Remap(MapTriangleID.Value);
				}
			}
			else
			{
				for (const int32 TriangleID : InMesh.TriangleIndicesItr())
				{
					Remap(TriangleID);
				}
			}
		}
	}

	bool ConvertDataToDynMeshes(const UPCGData* InData, FPCGContext* Context, TArray<UDynamicMesh*>& OutMeshes, bool bMergeMeshes, UGeometryScriptDebug* DynamicMeshDebug)
	{
		if (!InData)
		{
			return false;
		}

		auto CreateDynMeshFromGenerator = [&OutMeshes, Context](auto& Generator, const FTransform& Transform)
		{
			UDynamicMesh* NewMesh = OutMeshes.Emplace_GetRef(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context));
			FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();
			DynMesh.Copy(&Generator);
			MeshTransforms::ApplyTransform(DynMesh, Transform, /*bReverseOrientationIfNeeded=*/true);
		};

		auto CreateBoxDynMesh = [](const FBox& Box, const FTransform& Transform, FDynamicMesh3& OutMesh)
		{
			UE::Geometry::FMinimalBoxMeshGenerator Generator;
			UE::Geometry::FOrientedBox3d OrientedBox;
			OrientedBox.Frame = UE::Geometry::FFrame3d(Box.GetCenter());
			OrientedBox.Extents = Box.GetExtent();
			Generator.Box = OrientedBox;
			
			Generator.Generate();

			OutMesh.Copy(&Generator);
			MeshTransforms::ApplyTransform(OutMesh, Transform, /*bReverseOrientationIfNeeded=*/true);
		};

		auto AddBoxDynMesh = [&CreateBoxDynMesh, &OutMeshes, Context](const FBox& Box, const FTransform& Transform)
		{
			UDynamicMesh* NewMesh = OutMeshes.Emplace_GetRef(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context));
			FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();
			CreateBoxDynMesh(Box, Transform, DynMesh);
		};

		if (const UPCGPrimitiveData* PrimitiveData = Cast<UPCGPrimitiveData>(InData))
		{
			TWeakObjectPtr<UPrimitiveComponent> PrimitivePtr = PrimitiveData->GetComponent();
			if (!PrimitivePtr.IsValid())
			{
				return false;
			}

			// Convert from scene component to mesh and begin boolean operation
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(PrimitivePtr.Pin().Get()))
			{
				EGeometryScriptOutcomePins Outcome;
				FTransform PrimitiveTransform;
				// TODO: If we have repeats, ie. components that match content/transform, we could skip them.
				UGeometryScriptLibrary_SceneUtilityFunctions::CopyMeshFromComponent(
					SceneComponent,
					OutMeshes.Emplace_GetRef(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context)),
					PCGGeometryHelpers::Constants::CopyMeshFromComponentOptions,
					/*bTransformToWorld=*/true,
					PrimitiveTransform,
					Outcome,
					DynamicMeshDebug);

				if (Outcome != EGeometryScriptOutcomePins::Success)
				{
					OutMeshes.Pop();
					return false;
				}
			}
		}
		else if (const UPCGVolumeData* VolumeData = Cast<UPCGVolumeData>(InData))
		{
			// There are two cases here: either with the brush component or otherwise the simple volume box
			if (TStrongObjectPtr<AVolume> VolumeActor = VolumeData->GetVolumeActor().Pin())
			{
				EGeometryScriptOutcomePins Outcome;
				FTransform PrimitiveTransform;

				UGeometryScriptLibrary_SceneUtilityFunctions::CopyCollisionMeshesFromObject(
					VolumeActor->GetBrushComponent(),
					OutMeshes.Emplace_GetRef(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context)),
					/*bTransformToWorld=*/true,
					PrimitiveTransform,
					Outcome,
					/*bUseComplexCollision=*/false,
					/*SphereResolution*/PCGGeometryHelpers::Constants::SphereNumPhi,
					DynamicMeshDebug);

				if (Outcome != EGeometryScriptOutcomePins::Success)
				{
					OutMeshes.Pop();
					return false;
				}
			}
			else
			{
				AddBoxDynMesh(VolumeData->GetBounds(), FTransform::Identity);
			}
		}
		else if (const UPCGCollisionShapeData* ShapeData = Cast<UPCGCollisionShapeData>(InData))
		{
			const FTransform& ShapeTransform = ShapeData->GetTransform();
			const FCollisionShape& CollisionShape = ShapeData->GetCollisionShape();

			// Similar to FChaosVDGeometryBuilder::CreateMeshGeneratorForImplicitObject and FMakeSphereMeshDataflowNode::Evaluate
			if (CollisionShape.IsBox())
			{
				FBox CollisionShapeBox = FBox::BuildAABB(FVector::ZeroVector, CollisionShape.GetBox());
				AddBoxDynMesh(CollisionShapeBox, ShapeTransform);
			}
			else if (CollisionShape.IsSphere())
			{
				UE::Geometry::FSphereGenerator Generator;
				Generator.Radius = CollisionShape.GetSphereRadius();
				Generator.NumTheta = PCGGeometryHelpers::Constants::SphereNumTheta;
				Generator.NumPhi = PCGGeometryHelpers::Constants::SphereNumPhi;
				Generator.bPolygroupPerQuad = false;

				Generator.Generate();
				CreateDynMeshFromGenerator(Generator, ShapeTransform);
			}
			else if (CollisionShape.IsCapsule())
			{
				UE::Geometry::FCapsuleGenerator Generator;
				Generator.Radius = CollisionShape.GetCapsuleRadius();
				Generator.SegmentLength = 2.0 * CollisionShape.GetCapsuleAxisHalfLength();
				Generator.NumHemisphereArcSteps = PCGGeometryHelpers::Constants::CapsuleNumHemisphereSteps;
				Generator.NumCircleSteps = PCGGeometryHelpers::Constants::CapsuleNumCircleSteps;
				Generator.bPolygroupPerQuad = false;

				Generator.Generate();
				CreateDynMeshFromGenerator(Generator, ShapeTransform);
			}
			else
			{
				return false;
			}
		}
		else if (const UPCGCollisionWrapperData* CollisionWrapperData = Cast<UPCGCollisionWrapperData>(InData))
		{
			const EPCGCollisionQueryFlag CollisionQueryFlag = CollisionWrapperData->GetCollisionFlag();
			const FPCGCollisionWrapper& CollisionWrapper = CollisionWrapperData->GetCollisionWrapper();
			const UPCGBasePointData* PointData = CollisionWrapperData->GetPointData();
			check(PointData);
			const TConstPCGValueRange<FTransform> PointTransforms = PointData->GetConstTransformValueRange();
			const TConstPCGValueRange<FVector> PointMinBounds = PointData->GetConstBoundsMinValueRange();
			const TConstPCGValueRange<FVector> PointMaxBounds = PointData->GetConstBoundsMaxValueRange();

			UDynamicMesh* CurrentMesh = nullptr;

			// @todo_pcg: we probably should create only one FDynamicMesh per unique body instance, then do transformations.
			for (int PointIndex = 0; PointIndex < PointTransforms.Num(); ++PointIndex)
			{
				const FTransform& PointTransform = PointTransforms[PointIndex];
				const FBodyInstance* BodyInstance = CollisionWrapper.GetBodyInstance(PointIndex);

				// Implementation note: this is an adaptation of UGeometryScriptLibrary_SceneUtilityFunctions::CopyCollisionMeshesFromObject
				FDynamicMesh3 AccumulatedMesh;
				bool bHasData = false;

				if (BodyInstance)
				{
					const UBodySetup* BodySetup = BodyInstance->GetBodySetup();
					if (!BodySetup)
					{
						continue;
					}

					const int SphereResolution = 16;
					FVector ExternalScale = PointTransform.GetScale3D();

					UE::Geometry::FTransformSequence3d Transforms;
					FTransform PointTransformNoScale = PointTransform;
					PointTransformNoScale.SetScale3D(FVector::OneVector);
					Transforms.Append(PointTransformNoScale);

					// @todo_pcg: review this. Normally we should take the path that the shape is extracted from assumedly
					if (CollisionQueryFlag == EPCGCollisionQueryFlag::Complex)
					{
						// Requires a IInterface_CollisionDataProvider - see UE::Geometry::ConvertComplexCollisionToMeshes
					}
					else
					{
						UE::Geometry::ConvertSimpleCollisionToMeshes(BodySetup->AggGeom, AccumulatedMesh, Transforms, SphereResolution, false, false, nullptr, false, ExternalScale);
						bHasData = true;
					}
				}
				else
				{
					// Consider it's a box then
					FBox PointBox(PointMinBounds[PointIndex], PointMaxBounds[PointIndex]);
					CreateBoxDynMesh(PointBox, PointTransform, AccumulatedMesh);
					bHasData = true;
				}

				if (bHasData)
				{
					if (!CurrentMesh)
					{
						CurrentMesh = FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context);
						OutMeshes.Emplace(CurrentMesh);
					}

					if (bMergeMeshes)
					{
						CurrentMesh->EditMesh([&AccumulatedMesh](FDynamicMesh3& EditMesh)
						{
							EditMesh.AppendWithOffsets(AccumulatedMesh);
						});
					}
					else
					{
						CurrentMesh->SetMesh(MoveTemp(AccumulatedMesh));
						CurrentMesh = nullptr;
					}
				}
			}
		}
		else if (const UPCGDynamicMeshData* DynamicMeshData = Cast<UPCGDynamicMeshData>(InData))
		{
			if (DynamicMeshData->GetDynamicMesh())
			{
				UDynamicMesh* NewMesh = OutMeshes.Emplace_GetRef(FPCGContext::NewObject_AnyThread<UDynamicMesh>(Context));
				NewMesh->SetMesh(DynamicMeshData->GetDynamicMesh()->GetMeshRef());
			}
		}

		return true;
	}
} // namespace PCGGeometryHelpers