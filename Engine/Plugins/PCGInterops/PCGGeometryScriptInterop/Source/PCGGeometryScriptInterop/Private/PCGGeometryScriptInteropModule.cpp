// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "PCGModule.h"
#include "Data/PCGDynamicMeshData.h"
#include "Elements/PCGGetDynamicMeshData.h"

#include "Misc/CoreDelegates.h"

#if WITH_EDITOR
#include "Editor/PCGDynamicMeshDataVisualization.h"
#endif // WITH_EDITOR

class FPCGGeometryScriptInteropModule final : public IModuleInterface
{
public:
	void PreExit()
	{
		// No need to unregister if the PCG module is already dead.
		if (FPCGModule::IsPCGModuleLoaded())
		{
#if WITH_EDITOR
			FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetMutablePCGDataVisualizationRegistry();
			DataVisRegistry.UnregisterPCGDataVisualization(UPCGDynamicMeshData::StaticClass());
#endif // WITH_EDITOR

			FPCGGetDataFunctionRegistry& PCGDataFunctionRegistry = FPCGModule::MutableGetDataFunctionRegistry();
			PCGDataFunctionRegistry.UnregisterDataFromActorFunction(GetActorDataFunctionHandle);
			PCGDataFunctionRegistry.UnregisterDataFromComponentFunction(GetComponentDataFunctionHandle);
		}
	}

	//~ IModuleInterface implementation
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModuleChecked(TEXT("PCG"));

#if WITH_EDITOR
		FPCGDataVisualizationRegistry& DataVisRegistry = FPCGModule::GetMutablePCGDataVisualizationRegistry();
		DataVisRegistry.RegisterPCGDataVisualization(UPCGDynamicMeshData::StaticClass(), MakeUnique<const FPCGDynamicMeshDataVisualization>());
#endif // WITH_EDITOR

		FPCGGetDataFunctionRegistry& PCGDataFunctionRegistry = FPCGModule::MutableGetDataFunctionRegistry();
		GetActorDataFunctionHandle = PCGDataFunctionRegistry.RegisterDataFromActorFunction(&PCGGetDynamicMeshData::GetDynamicMeshDataFromActor);
		GetComponentDataFunctionHandle = PCGDataFunctionRegistry.RegisterDataFromComponentFunction(&PCGGetDynamicMeshData::GetDynamicMeshDataFromComponent);

		// Register onto the PreExit, because we need the class to be still valid to remove them from the mapping
		FCoreDelegates::OnPreExit.AddRaw(this, &FPCGGeometryScriptInteropModule::PreExit);
	}

	virtual void ShutdownModule() override
	{
		FCoreDelegates::OnPreExit.RemoveAll(this);
	}
	//~ End IModuleInterface implementation
	
private:
	FPCGGetDataFunctionRegistry::FFunctionHandle GetActorDataFunctionHandle = (uint64)(-1);
	FPCGGetDataFunctionRegistry::FFunctionHandle GetComponentDataFunctionHandle = (uint64)(-1);
};

IMPLEMENT_MODULE(FPCGGeometryScriptInteropModule, PCGGeometryScriptInterop);
