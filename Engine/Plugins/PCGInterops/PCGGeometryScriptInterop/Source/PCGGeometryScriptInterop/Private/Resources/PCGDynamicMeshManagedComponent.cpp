// Copyright Epic Games, Inc. All Rights Reserved.

#include "Resources/PCGDynamicMeshManagedComponent.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "Data/PCGDynamicMeshData.h"
#include "Helpers/PCGHelpers.h"

#include "Components/DynamicMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGDynamicMeshManagedComponent)

#define LOCTEXT_NAMESPACE "PCGDynamicMeshManagedComponent"

void UPCGDynamicMeshManagedComponent::MarkAsReused()
{
	Super::MarkAsReused();
	
	// We need to reset the transform is we re-use the component. Similar to ISMC code.
	if (UDynamicMeshComponent* Component = GetComponent())
	{
		FVector TentativeRootLocation = FVector::ZeroVector;
		
		if (USceneComponent* RootComponent = Component->GetAttachmentRoot())
		{
			TentativeRootLocation = RootComponent->GetComponentLocation();
		}

		// Since this is technically 'moving' the component, we need to unregister it before moving otherwise we could get a warning that we're moving a component with static mobility
		Component->UnregisterComponent();
		Component->SetWorldTransform(FTransform(FQuat::Identity, TentativeRootLocation, FVector::OneVector));
		Component->RegisterComponent();
	}
}

UDynamicMeshComponent* UPCGDynamicMeshManagedComponent::GetComponent() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGDynamicMeshManagedComponent::GetComponent);
	return Cast<UDynamicMeshComponent>(GeneratedComponent.Get());
}

void UPCGDynamicMeshManagedComponent::SetComponent(UDynamicMeshComponent* InComponent)
{
	GeneratedComponent = InComponent;
}

UPCGDynamicMeshManagedComponent* PCGDynamicMeshManagedComponent::GetOrCreateDynamicMeshManagedComponent(FPCGContext* Context, const UPCGSettingsInterface* SettingsInterface, const UPCGDynamicMeshData* InMeshData, AActor* TargetActor, TOptional<EPCGEditorDirtyMode> OptionalDirtyModeOverride)
{
	check(InMeshData);
	
	if (!TargetActor)
	{
		PCGLog::LogErrorOnGraph(LOCTEXT("NoTargetActor", "Cannot execute debug display for Dynamic Mesh data with no target actor."), Context);
		return nullptr;
	}

	UPCGComponent* SourceComponent = Cast<UPCGComponent>(Context->ExecutionSource.Get());
	if (!SourceComponent || !SettingsInterface)
	{
		return nullptr;
	}
	
	const FPCGCrc CRC = InMeshData->GetOrComputeCrc(/*bFullDataCrc=*/true);

	UPCGDynamicMeshManagedComponent* ExistingResource = nullptr;
	SourceComponent->ForEachManagedResource([&ExistingResource, &CRC, TargetActor](UPCGManagedResource* Resource)
	{
		// If we already found a valid resource, just skip until the end.
		if (ExistingResource)
		{
			return;
		}
		
		UPCGDynamicMeshManagedComponent* DynMeshResource = Cast<UPCGDynamicMeshManagedComponent>(Resource);
		if (!DynMeshResource)
		{
			return;
		}
		
		if (DynMeshResource->GetDataUID() == CRC.GetValue() && DynMeshResource->CanBeUsed())
		{
			UDynamicMeshComponent* Component = DynMeshResource->GetComponent();
			// Make sure the component is valid and on the right target actor.
			if (Component && Component->GetOwner() == TargetActor)
			{
				ExistingResource = DynMeshResource;
			}
		}
	});

	if (!ExistingResource)
	{
		ExistingResource = FPCGContext::NewObject_AnyThread<UPCGDynamicMeshManagedComponent>(Context, SourceComponent);
		SourceComponent->AddToManagedResources(ExistingResource);
		
		ExistingResource->SetDataUID(CRC.GetValue());
	}

	check(ExistingResource);
	
	ExistingResource->MarkAsUsed();
	const EPCGEditorDirtyMode DirtyMode = OptionalDirtyModeOverride.IsSet() ? OptionalDirtyModeOverride.GetValue() : SourceComponent->GetEditingMode();
	
#if WITH_EDITOR
	ExistingResource->ChangeTransientState(DirtyMode);
#endif // WITH_EDITOR

	UDynamicMeshComponent* DynMeshComponent = ExistingResource->GetComponent();
	if (!DynMeshComponent)
	{
		DynMeshComponent = FPCGContext::NewObject_AnyThread<UDynamicMeshComponent>(Context, TargetActor);
		if (DirtyMode == EPCGEditorDirtyMode::Preview)
		{
			DynMeshComponent->SetFlags(EObjectFlags::RF_Transient);
		}

		DynMeshComponent->RegisterComponent();
		TargetActor->AddInstanceComponent(DynMeshComponent);

		// Mimicking Static mesh managed resources
		if (!DynMeshComponent->AttachToComponent(TargetActor->GetRootComponent(), FAttachmentTransformRules(EAttachmentRule::KeepRelative, EAttachmentRule::KeepWorld, EAttachmentRule::KeepWorld, false)))
		{
			PCGLog::Component::LogComponentAttachmentFailedWarning(Context);
		}
		
		ExistingResource->SetComponent(DynMeshComponent);
	}

	// Add default tags. It's the callee responsibility to re-apply other tags.
	DynMeshComponent->ComponentTags.AddUnique(SourceComponent->GetFName());
	DynMeshComponent->ComponentTags.AddUnique(PCGHelpers::DefaultPCGTag);
	
	return ExistingResource;
}

#undef LOCTEXT_NAMESPACE
