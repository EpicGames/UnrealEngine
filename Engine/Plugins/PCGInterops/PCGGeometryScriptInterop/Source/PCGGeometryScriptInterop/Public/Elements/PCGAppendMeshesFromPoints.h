// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Elements/PCGDynamicMeshBaseElement.h"

#include "GeometryScript/GeometryScriptTypes.h"

#include "PCGAppendMeshesFromPoints.generated.h"

UENUM(Blueprintable)
enum class EPCGAppendMeshesFromPointsMode : uint8
{
	SingleStaticMesh UMETA(DisplayName="SingleMesh", ToolTip="Mesh taken from the node settings"),
	StaticMeshFromAttribute UMETA(DisplayName="MeshFromAttribute", ToolTip="Mesh taken from attributes on the points"),
	DynamicMesh UMETA(ToolTip="Mesh taken from another dynamic mesh"),
};

/**
* Append meshes at the points transforms. Mesh can be a single static/skeletal mesh, multiple meshes coming from the points or another dynamic mesh.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGAppendMeshesFromPointsSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return Mode != EPCGAppendMeshesFromPointsMode::DynamicMesh; };
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
#if WITH_EDITOR
	virtual EPCGChangeType GetChangeTypeForProperty(const FName& InPropertyName) const override;
#endif // WITH_EDITOR
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings")
	EPCGAppendMeshesFromPointsMode Mode;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = "Mode==EPCGAppendMeshesFromPointsMode::SingleStaticMesh", EditConditionHides, PCG_Overridable, PCG_OverrideAliases="StaticMesh", AllowedClasses="/Script/Engine.StaticMesh, /Script/Engine.SkeletalMesh"))
	TSoftObjectPtr<UObject> Mesh;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (EditCondition = "Mode==EPCGAppendMeshesFromPointsMode::StaticMeshFromAttribute", EditConditionHides, PCG_Overridable))
	FPCGAttributePropertyInputSelector MeshAttribute;
	
	/** Allows to extract materials from the static/skeletal mesh and set them in the resulting append. */
    UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (EditCondition = "Mode!=EPCGAppendMeshesFromPointsMode::DynamicMesh", EditConditionHides, PCG_Overridable))
    bool bExtractMaterials = true;
	
	/** LOD type to use when creating DynamicMesh from specified StaticMesh/SkeletalMesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|LODSettings", meta = (EditCondition = "Mode!=EPCGAppendMeshesFromPointsMode::DynamicMesh", EditConditionHides, PCG_Overridable))
	EGeometryScriptLODType RequestedLODType = EGeometryScriptLODType::RenderData;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|LODSettings", meta = (EditCondition = "Mode!=EPCGAppendMeshesFromPointsMode::DynamicMesh", EditConditionHides, PCG_Overridable))
	int32 RequestedLODIndex = 0;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug", meta = (EditCondition = "Mode!=EPCGAppendMeshesFromPointsMode::DynamicMesh", EditConditionHides))
	bool bSynchronousLoad = false;

	UE_DEPRECATED(5.8, "Use Mesh property")
	TSoftObjectPtr<UStaticMesh> StaticMesh;
};

struct FPCGAppendMeshesFromPointsContext : public FPCGContext, public IPCGAsyncLoadingContext
{
	bool bPrepareDataSucceeded = false;
	TMap<FSoftObjectPath, TArray<int32>> MeshToPointIndicesMapping;
};

class FPCGAppendMeshesFromPointsElement : public IPCGDynamicMeshBaseElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

