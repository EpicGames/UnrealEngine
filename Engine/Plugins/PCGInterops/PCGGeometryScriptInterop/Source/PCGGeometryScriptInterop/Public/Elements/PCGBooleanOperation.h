// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGDynamicMeshBaseElement.h"

#include "GeometryScript/MeshBooleanFunctions.h"

#include "PCGBooleanOperation.generated.h"

UENUM(Blueprintable)
enum class EPCGBooleanOperationTagInheritanceMode : uint8
{
	Both,
	A,
	B,
};

UENUM(Blueprintable)
enum class EPCGBooleanOperationMode : uint8
{
	EachAWithEachB UMETA(DisplayName="Each A With Each B", Tooltip="Each input in A is boolean'd with its associated input in B (A1 with B1, A2 with B2, etc...). Produces N outputs."),
	EachAWithEachBSequentially UMETA(DisplayName="Each A With Each B Sequentially", Tooltip="Each input in A is boolean'd with every input in B sequentially. (A1 with B1 then with B2, A2 with B1 then B2, etc...). Produces N outputs."),
	EachAWithEveryB UMETA(DisplayName="Each A With Every B", Tooltip="Each input in A is boolean'd with input in B individually (Cartesian product: A1 with B1, A1 with B2, A2 with B1, A2 with B2, etc...). Produces N * M outputs.")
};

/**
* Do a boolean operation between dynamic meshes.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGBooleanOperationSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
	
public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EGeometryScriptBooleanOperation BooleanOperation = EGeometryScriptBooleanOperation::Intersection;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FGeometryScriptMeshBooleanOptions BooleanOperationOptions;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGBooleanOperationTagInheritanceMode TagInheritanceMode;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	EPCGBooleanOperationMode Mode = EPCGBooleanOperationMode::EachAWithEachB;
};

class FPCGBooleanOperationElement : public IPCGDynamicMeshBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

