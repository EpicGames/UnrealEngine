// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGDynamicMeshBaseElement.h"

#include "PCGCreateEmptyDynamicMesh.generated.h"

/**
* Create an empty dynamic mesh data.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGCreateEmptyDynamicMeshSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGCreateEmptyDynamicMeshElement : public IPCGDynamicMeshBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

