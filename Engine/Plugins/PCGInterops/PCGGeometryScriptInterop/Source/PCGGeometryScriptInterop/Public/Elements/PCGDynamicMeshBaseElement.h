// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGDynamicMeshBaseElement.generated.h"

struct FPCGContext;
class UPCGDynamicMeshData;

/**
* Base class for Dynamic Mesh nodes
*/
UCLASS(Abstract, MinimalAPI, ClassGroup = (Procedural))
class UPCGDynamicMeshBaseSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::DynamicMesh; }
#endif

protected:
	PCGGEOMETRYSCRIPTINTEROP_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	PCGGEOMETRYSCRIPTINTEROP_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface
};

/**
* Base class for Dynamic Mesh elements
*/
class IPCGDynamicMeshBaseElement : public IPCGElement
{
protected:
	/** All dyn mesh elements are not cacheable to benefit from stealing data. */
	virtual bool IsCacheable(const UPCGSettings*) const override { return false; }
	virtual bool ShouldVerifyIfOutputsAreUsedMultipleTimes(const UPCGSettings*) const override { return true; }

public:
	PCGGEOMETRYSCRIPTINTEROP_API static UPCGDynamicMeshData* CopyOrSteal(const FPCGTaggedData& InTaggedData, FPCGContext* InContext);
};

template <typename ContextType> requires std::is_base_of_v<FPCGContext, ContextType>
class IPCGDynamicMeshBaseElementWithCustomContext : public IPCGDynamicMeshBaseElement
{
protected:
	virtual FPCGContext* CreateContext() override { return new ContextType{}; }
};
