// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGDynamicMeshBaseElement.h"

#include "PCGDynamicMeshTransform.generated.h"

/**
* Apply a transform to all dynamic meshes.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGDynamicMeshTransformSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
	
public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	FTransform Transform;
};

class FPCGDynamicMeshTransformElement : public IPCGDynamicMeshBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

