// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGExecuteBlueprint.h"

#include "PCGGeometryBlueprintElement.generated.h"

class UDynamicMesh;
class UPCGDynamicMeshData;

/**
* Subclass of PCG Blueprint Base Element, it comes with pre-configured pins as input and output for Dynamic meshes and force to be non-cacheable.
* The function CopyOrStealInputData is a helper to either steal (efficient) or copy the input data (less efficient) so work can be done in place on the Dynamic Mesh.
* More importantly, a user deriving from this class will want to implement ProcessDynamicMesh, the only thing needed to streamline the process and
* removes all the boilerplate when in a simple input->output case.
*/
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable, hidecategories = (Object))
class UPCGGeometryBlueprintElement : public UPCGBlueprintBaseElement
{
	GENERATED_BODY()
	
public:
	PCGGEOMETRYSCRIPTINTEROP_API UPCGGeometryBlueprintElement();

	UE_DEPRECATED(5.7, "Use Execute_Implementation instead")
	virtual void ExecuteWithContext_Implementation(FPCGContext& InContext, const FPCGDataCollection& Input, FPCGDataCollection& Output) {}

	/** Virtual implementation to streamline the creation of a Geometry Script node processing. Do not override this function or Execute if you want to use the streamlined version. */
	PCGGEOMETRYSCRIPTINTEROP_API virtual void Execute_Implementation(const FPCGDataCollection& Input, FPCGDataCollection& Output) override;

	/**
     * Streamlined version of the Execute function, to only process the dynamic meshes.
     * For each input that is a dynamic mesh, we will call this function, and it will create as many output data as there are inputs.
     * @param InDynMesh  - Dynamic mesh to process. Can be used as is and do operation in place.
     * @param OutTags  - Optional tags to add to the output. By default, it will inherit the tags of the input.
     */
    UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, Category = "PCG|Execution", meta = (AutoCreateRefTerm = "InTags"))
	PCGGEOMETRYSCRIPTINTEROP_API void ProcessDynamicMesh(UDynamicMesh* InDynMesh, TArray<FString>& OutTags);

	/** Allows to steal the data and work in place if the data is not used elsewhere. If this element is cacheable, it will automatically copy. */
	UFUNCTION(BlueprintCallable, Category = "PCG|DynamicMesh", meta = (HidePin = "Self"))
	PCGGEOMETRYSCRIPTINTEROP_API UPCGDynamicMeshData* CopyOrStealInputData(const FPCGTaggedData& InTaggedData) const;
};
