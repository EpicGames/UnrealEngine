// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGDataFromActor.h"

#include "GeometryScript/SceneUtilityFunctions.h"

#include "PCGGetDynamicMeshData.generated.h"

struct FPCGGetDataFunctionRegistryOutput;
struct FPCGGetDataFunctionRegistryParams;

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetDynamicMeshDataSettings : public UPCGDataFromActorSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface
	
	//~Begin UPCGDataFromActorSettings interface
public:
	virtual EPCGDataType GetDataFilter() const override { return EPCGDataType::DynamicMesh; }

protected:
#if WITH_EDITOR
	virtual bool DisplayModeSettings() const override { return false; }
#endif
	//~End UPCGDataFromActorSettings
	
public:
	/** If data is coming from a component, you can impact the options there. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Dynamic Mesh Settings", meta = (PCG_Overridable, ShowOnlyInnerProperties))
	FGeometryScriptCopyMeshFromComponentOptions Options;
};

namespace PCGGetDynamicMeshData
{
	PCGGEOMETRYSCRIPTINTEROP_API bool GetDynamicMeshDataFromActor(FPCGContext*, const FPCGGetDataFunctionRegistryParams&, AActor*, FPCGGetDataFunctionRegistryOutput&);
	PCGGEOMETRYSCRIPTINTEROP_API bool GetDynamicMeshDataFromComponent(FPCGContext*, const FPCGGetDataFunctionRegistryParams&, UActorComponent*, FPCGGetDataFunctionRegistryOutput&);
}