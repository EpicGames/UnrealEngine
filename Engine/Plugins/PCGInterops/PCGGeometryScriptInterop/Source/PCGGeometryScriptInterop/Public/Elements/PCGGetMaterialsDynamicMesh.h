// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGDynamicMeshBaseElement.h"

#include "PCGGetMaterialsDynamicMesh.generated.h"

/**
* Retrieve the array of materials on a dynamic mesh data.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGGetMaterialsDynamicMeshSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	//~End UPCGSettings interface
};

class FPCGGetMaterialsDynamicMeshElement : public IPCGDynamicMeshBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

