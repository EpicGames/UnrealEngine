// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGDynamicMeshBaseElement.h"

#include "PCGMergeDynamicMeshes.generated.h"

/**
* Appends all incoming dynamic meshes to the first dynamic mesh in order.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMergeDynamicMeshesSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
};

class FPCGMergeDynamicMeshesElement : public IPCGDynamicMeshBaseElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

