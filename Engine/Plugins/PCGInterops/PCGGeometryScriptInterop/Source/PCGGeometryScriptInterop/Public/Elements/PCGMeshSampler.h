// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "PCGElement.h"
#include "PCGSettings.h"
#include "Elements/PCGLoadObjectsContext.h"

#include "Async/Future.h"
#include "GeometryScript/GeometryScriptTypes.h"
#include "GeometryScript/MeshSamplingFunctions.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

#include <atomic>

#include "PCGMeshSampler.generated.h"

#define UE_API PCGGEOMETRYSCRIPTINTEROP_API

template <typename T>
class FPCGMetadataAttribute;

class FProgressCancel;
class UDynamicMesh;
class UPCGPointData;
class UStaticMesh;

UENUM(BlueprintType)
enum class EPCGMeshSamplingMethod : uint8
{
	/** Sample one point (at the center) of each triangle of the mesh. */
	OnePointPerTriangle,

	/** Sample one point per vertex on the mesh. */
	OnePointPerVertex,

	/** Use Poisson sampling to sample points on the mesh. Can be expensive and therefore it is not framebound. */
	PoissonSampling
};

UENUM()
enum class EPCGColorChannel
{
	Red,
	Green,
	Blue,
	Alpha
};

/**
* Sample points on a mesh.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshSamplerSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UE_API UPCGMeshSamplerSettings();

	//~Begin UObject interface
	virtual void PostLoad() override;
	//~End UObject interface

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("MeshSampler")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGMeshSamplerSettings", "NodeTitle", "Mesh Sampler"); }
	UE_API virtual FText GetNodeTooltipText() const override;
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Sampler; }
	virtual bool CanDynamicallyTrackKeys() const override { return bExtractMeshFromInput; }
	UE_API virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
#endif
	virtual bool UseSeed() const override { return true; }
	UE_API virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	UE_API virtual TArray<FPCGPinProperties> OutputPinProperties() const override;

protected:
	UE_API virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	/** 
	* Can provide a list of inputs to sample the meshes from. It can be a list of StaticMeshes/SkeletalMeshes, a list of Actors that have a Scene Component (like a Static Mesh Component), or a list of Scene Components directly.
	* Geometry Script needs to be able to extract a dynamic mesh from this scene component (so won't work for ISMCs for example) and for now will work only with a single scene component.
	* Each entry (either in the same data or separate data) will produce a unique output data.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	bool bExtractMeshFromInput = false;

	/** Soft Object Path to the mesh to sample from. Supports Static Meshes and Skeletal Meshes. Will be loaded. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases="StaticMesh", AllowedClasses="/Script/Engine.StaticMesh, /Script/Engine.SkeletalMesh", EditCondition = "!bExtractMeshFromInput", EditConditionHides, ShowAfter = "bExtractMeshFromInput"))
	TSoftObjectPtr<UObject> Mesh;

	/** Selector to read data from. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (PCG_Overridable, EditCondition = "bExtractMeshFromInput", EditConditionHides, ShowAfter = "bExtractMeshFromInput"))
	FPCGAttributePropertyInputSelector InputSource;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	EPCGMeshSamplingMethod SamplingMethod = EPCGMeshSamplingMethod::OnePointPerTriangle;

	/** Will extract the color channel into the density. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Color & Density", meta = (PCG_Overridable, InlineEditConditionToggle, PCG_OverrideAliases="bUseRedAsDensity"))
	bool bUseColorChannelAsDensity = false;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Color & Density", meta = (PCG_Overridable, EditCondition="bUseColorChannelAsDensity"))
	EPCGColorChannel ColorChannelAsDensity = EPCGColorChannel::Red;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Color & Density", meta = (InlineEditConditionToggle, PCG_Overridable))
	bool bHasDefaultColorOverride = true;

	/** When the mesh doesn't have any vertex color, will use this color instead. Otherwise, it will use (0,0,0,0) for "per point" and white for the rest. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Color & Density", meta = (PCG_Overridable, EditCondition=bHasDefaultColorOverride))
	FLinearColor DefaultColorOverride = FLinearColor::White;

	/** Enable voxelisation as a preparation pass. Can be more expensive given the VoxelSize. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Voxelize Options", meta = (PCG_Overridable))
	bool bVoxelize = false;

	/** Size of a voxel for the voxelization. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Voxelize Options", meta = (PCG_Overridable, EditCondition = "bVoxelize"))
	float VoxelSize = 100.0f;

	/** Post-processing pass after voxelization to remove hidden triangles. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Voxelize Options", meta = (PCG_Overridable, EditCondition = "bVoxelize"))
	bool bRemoveHiddenTriangles = true;

	/** LOD type to use when creating DynamicMesh from specified StaticMesh/SkeletalMesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|LODSettings", meta = (PCG_Overridable))
	EGeometryScriptLODType RequestedLODType = EGeometryScriptLODType::RenderData;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|LODSettings", meta = (PCG_Overridable))
	int32 RequestedLODIndex = 0;

	// Poisson Sampling parameters
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Poisson sampling", meta = (PCG_Overridable, EditCondition = "SamplingMethod == EPCGMeshSamplingMethod::PoissonSampling"))
	FGeometryScriptMeshPointSamplingOptions SamplingOptions;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Poisson sampling", meta = (PCG_Overridable, EditCondition = "SamplingMethod == EPCGMeshSamplingMethod::PoissonSampling"))
	FGeometryScriptNonUniformPointSamplingOptions NonUniformSamplingOptions;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|UVs", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex"))
	bool bExtractUVAsAttribute = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|UVs", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex && bExtractUVAsAttribute", EditConditionHides))
	FName UVAttributeName = TEXT("UV");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|UVs", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex && bExtractUVAsAttribute", EditConditionHides))
	int32 UVChannel = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex"))
	bool bOutputTriangleIds = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex && bOutputTriangleIds", EditConditionHides))
	FName TriangleIdAttributeName = TEXT("TriangleId");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex"))
	bool bOutputMaterialInfo = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex && bOutputMaterialInfo", EditConditionHides))
	FName MaterialIdAttributeName = TEXT("MaterialId");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Extra", meta = (PCG_Overridable, EditCondition = "SamplingMethod != EPCGMeshSamplingMethod::OnePointPerVertex && bOutputMaterialInfo", EditConditionHides))
	FName MaterialAttributeName = TEXT("Material");

	/** Each PCG point represents a discretized, volumetric region of world space. The points' Steepness value [0.0 to
	 * 1.0] establishes how "hard" or "soft" that volume will be represented. From 0, it will ramp up linearly
	 * increasing its influence over the density from the point's center up to two times the bounds. At 1, it will
	 * represent a binary box function with the size of the point's bounds.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Points", meta=(ClampMin="0", ClampMax="1", PCG_Overridable))
	float PointSteepness = 0.5f;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

	UE_DEPRECATED(5.8, "Use Mesh property")
	TSoftObjectPtr<UStaticMesh> StaticMesh;
};

/**
* Extra context to store all the data that need to be kept between multiple executions (time slicing)
*/
struct FPCGMeshSamplerContext : public FPCGLoadObjectsFromPathContext
{
public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPCGMeshSamplerContext() = default;
	virtual ~FPCGMeshSamplerContext();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	using SetPointDensityFunc = void(*)(const FLinearColor&, FPCGPoint&);

	void SetAttributeValues(int32 UVChannel, int32 TriangleId, const FVector& BarycentricCoord, int64& MetadataEntry, int32 DataIndex);
	void SetPointColorAndDensity(SetPointDensityFunc SetPointDensityFuncPtr, int32 TriangleId, const FVector& BarycentricCoord, FPCGPoint& OutPoint, int32 DataIndex, TOptional<FLinearColor> DefaultColor);

protected:
	virtual void AddExtraStructReferencedObjects(FReferenceCollector& Collector) override;
	
public:
	// Dynamic meshes. Will be added to root.
	TArray<TObjectPtr<UDynamicMesh>> DynamicMeshes;

	// Lists extracted from the meshes
	TArray<FGeometryScriptVectorList> Positions;
	TArray<FGeometryScriptColorList> Colors;
	TArray<FGeometryScriptVectorList> Normals;
	TArray<FGeometryScriptIndexList> TriangleIds;

	// Output point data.
	UE_DEPRECATED(5.6, "Use OutputData instead")
	TArray<UPCGPointData*> OutPointData;

	TArray<UPCGBasePointData*> OutputData;

	// Optional attributes.
	TArray<FPCGMetadataAttribute<FVector2D>*> UVAttributes;
	TArray<FPCGMetadataAttribute<int32>*> TriangleIdAttributes;
	TArray<FPCGMetadataAttribute<int32>*> MaterialIdAttributes;
	TArray<FPCGMetadataAttribute<FSoftObjectPath>*> MaterialAttributes;

	// Material Specific
	TArray<TArray<UMaterialInterface*>> ComponentMaterialList;
	TArray<TArray<UMaterialInterface*>> AssetMaterialList;

	// For Poisson sampling, we are starting futures that are not framebound
	// Store the futures and synchronisation items in the context
	TArray<TFuture<bool>> SamplingFutures;
	std::atomic<bool> StopSampling = false;
	TUniquePtr<FProgressCancel> SamplingProgess;

	// Starting indices for each different object to sample. If we have 3 meshes of 20 elements (like vertices) each, the array will be [0, 20, 40, 60]
	TArray<int32> StartingIndices;

	// Set to true if prepared data succeeded.
	bool bDataPrepared = false;
};

class FPCGMeshSamplerElement : public IPCGElement
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }

protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
	virtual bool SupportsBasePointDataInputs(FPCGContext* InContext) const override { return true; }
};

#undef UE_API
