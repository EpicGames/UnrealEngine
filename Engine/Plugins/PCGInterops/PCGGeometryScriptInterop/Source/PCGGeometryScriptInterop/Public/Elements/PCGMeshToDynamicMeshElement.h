// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Elements/PCGDynamicMeshBaseElement.h"

#include "GeometryScript/GeometryScriptTypes.h"

#include "PCGMeshToDynamicMeshElement.generated.h"

/**
* Convert a static/skeletal mesh into a dynamic mesh data.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGMeshToDynamicMeshSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual void GetStaticTrackedKeys(FPCGSelectionKeyToSettingsMap& OutKeysToSettings, TArray<TObjectPtr<const UPCGGraph>>& OutVisitedGraphs) const override;
	virtual bool CanDynamicallyTrackKeys() const override { return true; };
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, PCG_OverrideAliases="StaticMesh", AllowedClasses="/Script/Engine.StaticMesh, /Script/Engine.SkeletalMesh"))
	TSoftObjectPtr<UObject> Mesh;

	/** Allows to extract materials from the static/skeletal mesh and store them in the PCG Dynamic Mesh Data. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	bool bExtractMaterials = true;

	/** If it extracts materials, we can specify override materials. It needs to have the same number of material overrides than there are materials on the static mesh. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bExtractMaterials", EditConditionHides, PCG_Overridable))
	TArray<TSoftObjectPtr<UMaterialInterface>> OverrideMaterials;

	/** LOD type to use when creating DynamicMesh from specified StaticMesh/SkeletalMesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|LODSettings", meta = (PCG_Overridable))
	EGeometryScriptLODType RequestedLODType = EGeometryScriptLODType::MaxAvailable;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|LODSettings", meta = (PCG_Overridable))
	int32 RequestedLODIndex = 0;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings|Debug")
	bool bSynchronousLoad = false;

	UE_DEPRECATED(5.8, "Use Mesh property")
	TSoftObjectPtr<UStaticMesh> StaticMesh;
};

struct FPCGMeshToDynamicMeshContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGMeshToDynamicMeshElement : public IPCGDynamicMeshBaseElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override;
protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareDataInternal(FPCGContext* Context) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

