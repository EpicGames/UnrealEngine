// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGSettings.h"

#include "PCGPrimitiveCrossSection.generated.h"

namespace PCGPrimitiveCrossSection::Constants
{
	static constexpr double MinTierMergingThreshold = 0.01;
}

/**
 * Creates spline cross-sections of one more primitives based on vertex features.
 */
UCLASS(BlueprintType, ClassGroup = (Procedural))
class UPCGPrimitiveCrossSectionSettings : public UPCGSettings
{
	GENERATED_BODY()

public:
	UPCGPrimitiveCrossSectionSettings();

	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("PrimitiveCrossSection")); }
	virtual FText GetDefaultNodeTitle() const override { return NSLOCTEXT("PCGPrimitiveCrossSectionElement", "NodeTitle", "Primitive Cross-Section"); }
	virtual FText GetNodeTooltipText() const override { return NSLOCTEXT("PCGPrimitiveCrossSectionElement", "NodeTooltip", "Creates spline cross-sections of one more primitives based on vertex features."); }
	virtual EPCGSettingsType GetType() const override { return EPCGSettingsType::Spatial; }
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	/** Slicing will happen from the minimum vertex along this direction vector (normalized). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FVector SliceDirection = FVector::UpVector;

	/** The attribute that will be populated with each cross-section's extrusion vector. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FPCGAttributePropertyOutputSelector ExtrusionVectorAttribute;

	/** The minimum required number of vertices that must be co-planar in order to be considered a tier "feature". */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	int32 MinimumCoplanarVertices = 3;

	/** A safeguard to prevent finding features on an overly complex mesh. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, AdvancedDisplay, meta = (PCG_Overridable, EditCondition = "GenerateCrossSectionMode != EPCGGenerateCrossSectionMode::TierSlicing", EditConditionHides))
	int32 MaxMeshVertexCount = 2048;

	/** Cull tiers that are within a specified threshold. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bEnableTierMerging = false;

	/** If a tier is within this distance (in cm) of the previous tier, it will be culled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ClampMin = "0.01", PCG_Overridable, EditCondition = "bEnableTierMerging"))
	double TierMergingThreshold = 1.0;

	/** Cull tiers that have a surface area smaller than a specified threshold. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bEnableMinAreaCulling = false;

	/** If a tier is smaller in area than this threshold, it will be culled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bEnableMinAreaCulling"))
	double MinAreaCullingThreshold = 100.0;

	/** Culls tiers that don't meed a minimum height requirement. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, InlineEditConditionToggle))
	bool bEnableMinHeightCulling = true;

	/** If a tier is smaller in height than this threshold, it will be culled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable, EditCondition = "bEnableMinHeightCulling"))
	double MinHeightCullingThreshold = 1.0;

	/** If multiple tiers can be combined into a single tier without affecting the contour, remove the redundant one. Note: This will currently cull even if there are other unique tiers in between. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bRemoveRedundantSections = true;
};

class FPCGPrimitiveCrossSectionElement : public IPCGElement
{
protected:
	virtual bool ExecuteInternal(FPCGContext* Context) const override;
	virtual bool IsCacheable(const UPCGSettings* InSettings) const override { return false; }
};
