// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGAssetExporter.h"
#include "Elements/PCGDynamicMeshBaseElement.h"

#include "GeometryScript/MeshAssetFunctions.h"

#include "PCGSaveDynamicMeshToAsset.generated.h"

/**
* Saves dynamic mesh data into a static mesh asset.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSaveDynamicMeshToAssetSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual TArray<FPCGPinProperties> InputPinProperties() const override;
	virtual TArray<FPCGPinProperties> OutputPinProperties() const override;
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
	
public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties, PCG_Overridable))
	FPCGAssetExporterParameters ExportParams;

	/**
	 * This option has higher priority than CopyMeshToAssetOptions.ReplaceMaterials.
	 * If true, we will replace the materials from the materials stored on the PCG Dynamic Mesh data.
	 * Otherwise, we will follow what is set in CopyMeshToAssetOptions.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	bool bExportMaterialsFromDynamicMesh = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FGeometryScriptCopyMeshToAssetOptions CopyMeshToAssetOptions;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (PCG_Overridable))
	FGeometryScriptMeshWriteLOD MeshWriteLOD;
};

class FPCGSaveDynamicMeshToAssetElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

