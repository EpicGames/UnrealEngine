// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGContext.h"
#include "Async/PCGAsyncLoadingContext.h"
#include "Elements/PCGDynamicMeshBaseElement.h"

#include "PCGSetMaterialsDynamicMesh.generated.h"

/**
* Replace the array of materials on the dynamic mesh data.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSetMaterialsDynamicMeshSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface
	
public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Settings", meta = (PCG_Overridable))
	TArray<TSoftObjectPtr<UMaterialInterface>> Materials;
};

struct FPCGSetMaterialsDynamicMeshContext : public FPCGContext, public IPCGAsyncLoadingContext {};

class FPCGSetMaterialsDynamicMeshElement : public IPCGDynamicMeshBaseElementWithCustomContext<FPCGSetMaterialsDynamicMeshContext>
{
public:
	// Loading needs to be done on the main thread and accessing objects outside of PCG might not be thread safe, so taking the safe approach
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext* Context) const override { return true; }
protected:
	virtual bool PrepareDataInternal(FPCGContext* InContext) const override;
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

