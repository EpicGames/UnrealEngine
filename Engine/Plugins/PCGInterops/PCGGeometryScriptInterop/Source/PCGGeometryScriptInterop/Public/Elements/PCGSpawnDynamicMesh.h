// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGDynamicMeshBaseElement.h"

#include "PCGSpawnDynamicMesh.generated.h"

class UDynamicMeshComponent;
/**
* Spawn a dynamic mesh component for each dynamic mesh data in input.
*/
UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGSpawnDynamicMeshSettings : public UPCGDynamicMeshBaseSettings
{
	GENERATED_BODY()

public:
	//~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override;
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	//~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, Category = Settings, meta = (PCG_Overridable))
	TSoftObjectPtr<AActor> TargetActor;

	/** Specify a list of functions to be called on the target actor after instances are spawned. Functions need to be parameter-less and with "CallInEditor" flag enabled. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TArray<FName> PostProcessFunctionNames;
};

class FPCGSpawnDynamicMeshElement : public IPCGElement
{
public:
	virtual bool CanExecuteOnlyOnMainThread(FPCGContext*) const override { return true; }
	virtual bool IsCacheable(const UPCGSettings*) const override { return false; }
protected:
	virtual bool ExecuteInternal(FPCGContext* InContext) const override;
};

