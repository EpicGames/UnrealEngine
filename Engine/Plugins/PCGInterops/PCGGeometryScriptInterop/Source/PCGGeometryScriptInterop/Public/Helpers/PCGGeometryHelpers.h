// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/PCGMeshSampler.h"

#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "GeometryScript/GeometryScriptTypes.h"

struct FPCGContext;
class UGeometryScriptDebug;
class UMaterialInterface;

namespace UE::Geometry
{
	class FDynamicMesh3;
	struct FMeshIndexMappings;
}

namespace PCGGeometryHelpers
{
	PCGGEOMETRYSCRIPTINTEROP_API void GeometryScriptDebugToPCGLog(FPCGContext* Context, const UGeometryScriptDebug* Debug);
	UE::Conversion::EMeshLODType SafeConversionLODType(const EGeometryScriptLODType LODType);

	/**
	 * Adaptation of UGeometryScriptLibrary_MeshMaterialFunctions::RemapToNewMaterialIDsByMaterial, to work on UE::Geometry::FDynamicMesh3 and with optional mappings.
	 * @param InMesh: Mesh to modify
	 * @param FromMaterials: Original array of materials for the mesh
	 * @param ToMaterials: New array of materials for the mesh. Mutable. If the original material is not in the new array, add it.
	 * @param OptionalMappings: Optional mappings if we want to update a subset of the triangles in the mesh.
	 */
	void RemapMaterials(UE::Geometry::FDynamicMesh3& InMesh, const TArray<UMaterialInterface*>& FromMaterials, TArray<TObjectPtr<UMaterialInterface>>& ToMaterials, const UE::Geometry::FMeshIndexMappings* OptionalMappings = nullptr);

	bool ConvertDataToDynMeshes(const UPCGData* InData, FPCGContext* Context, TArray<UDynamicMesh*>& OutMeshes, bool bMergeMeshes, UGeometryScriptDebug* DynamicMeshDebug = nullptr);
}
